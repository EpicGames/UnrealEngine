{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Procedural Mesh Component",
	"Description" : "A renderable component and library of utilities for creating and modifying mesh geometry procedurally.",
	"Category" : "Rendering",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed": false,
	"SupportedPrograms":  ["LiveLinkHub"],
	"Modules" :
	[
		{
			"Name": "ProceduralMeshComponent",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"PlatformDenyList": [
			]
		},
		{
			"Name": "ProceduralMeshComponentEditor",
			"Type": "Editor",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "Default"
		}
	]
}
