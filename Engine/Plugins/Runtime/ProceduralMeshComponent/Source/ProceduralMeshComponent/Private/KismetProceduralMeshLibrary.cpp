// Copyright Epic Games, Inc. All Rights Reserved. 

#include "KismetProceduralMeshLibrary.h"
#include "Components/StaticMeshComponent.h"
#include "PhysicsEngine/BodySetup.h"
#include "Materials/MaterialInterface.h"
#include "ProceduralMeshComponentPluginPrivate.h"
#include "ProceduralMeshComponent.h"
#include "StaticMeshResources.h"
#include "Engine/StaticMesh.h"
#include "GeomTools.h"

#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetProceduralMeshLibrary)

DECLARE_CYCLE_STAT(TEXT("Update Collision"), STAT_ProcMesh_CalcTangents, STATGROUP_ProceduralMesh);

#define LOCTEXT_NAMESPACE "KismetProcMeshLibrary"


void UKismetProceduralMeshLibrary::ConvertQuadToTriangles(TArray<int32>& Triangles, int32 Vert0, int32 Vert1, int32 Vert2, int32 Vert3)
{
	Triangles.Add(Vert0);
	Triangles.Add(Vert1);
	Triangles.Add(Vert3);

	Triangles.Add(Vert1);
	Triangles.Add(Vert2);
	Triangles.Add(Vert3);
}

void UKismetProceduralMeshLibrary::CreateGridMeshTriangles(int32 NumX, int32 NumY, bool bWinding, TArray<int32>& Triangles)
{
	Triangles.Reset();

	if (NumX >= 2 && NumY >= 2)
	{
		// Build Quads
		for (int XIdx = 0; XIdx < NumX - 1; XIdx++)
		{
			for (int YIdx = 0; YIdx < NumY - 1; YIdx++)
			{
				const int32 I0 = (XIdx + 0)*NumY + (YIdx + 0);
				const int32 I1 = (XIdx + 1)*NumY + (YIdx + 0);
				const int32 I2 = (XIdx + 1)*NumY + (YIdx + 1);
				const int32 I3 = (XIdx + 0)*NumY + (YIdx + 1);

				if (bWinding)
				{
					ConvertQuadToTriangles(Triangles, I0, I1, I2, I3);
				}
				else
				{
					ConvertQuadToTriangles(Triangles, I0, I3, I2, I1);
				}
			}
		}
	}
}

void UKismetProceduralMeshLibrary::CreateGridMeshWelded(int32 NumX, int32 NumY, TArray<int32>& Triangles, TArray<FVector>& Vertices, TArray<FVector2D>& UVs, float GridSpacing)
{
	Triangles.Empty();
	Vertices.Empty();
	UVs.Empty();

	if (NumX >= 2 && NumY >= 2)
	{
		FVector2D Extent = FVector2D((NumX - 1)* GridSpacing, (NumY - 1) * GridSpacing) / 2;

		for (int i = 0; i < NumY; i++)
		{
			for (int j = 0; j < NumX; j++)
			{
				Vertices.Add(FVector((float)j * GridSpacing - Extent.X, (float)i * GridSpacing - Extent.Y, 0));
				UVs.Add(FVector2D((float)j / ((float)NumX - 1), (float)i / ((float)NumY - 1)));
			}
		}

		for (int i = 0; i < NumY - 1; i++)
		{
			for (int j = 0; j < NumX - 1; j++)
			{
				int idx = j + (i * NumX);
				Triangles.Add(idx);
				Triangles.Add(idx + NumX);
				Triangles.Add(idx + 1);

				Triangles.Add(idx + 1);
				Triangles.Add(idx + NumX);
				Triangles.Add(idx + NumX + 1);
			}
		}
	}
}

void UKismetProceduralMeshLibrary::CreateGridMeshSplit(int32 NumX, int32 NumY, TArray<int32>& Triangles, TArray<FVector>& Vertices, TArray<FVector2D>& UVs, TArray<FVector2D>& UV1s, float GridSpacing)
{
	Triangles.Empty();
	Vertices.Empty();
	UVs.Empty();
	UV1s.Empty();

	if (NumX >= 2 && NumY >= 2)
	{

		FVector2D Extent = FVector2D(NumX * GridSpacing, NumY * GridSpacing) / 2;

		for (int i = 0; i < NumY - 1; i++)
		{
			for (int j = 0; j < NumX - 1; j++)
			{
				int idx = j + (i * (NumX - 1));
				Triangles.Add(idx * 4 + 3);
				Triangles.Add(idx * 4 + 1);
				Triangles.Add(idx * 4);

				Triangles.Add(idx * 4 + 3);
				Triangles.Add(idx * 4 + 2);
				Triangles.Add(idx * 4 + 1);

				float Z = FMath::Fmod(idx, 5.f) * GridSpacing;
				FVector CornerVert = FVector((float)j * GridSpacing - Extent.X, (float)i * GridSpacing - Extent.Y, Z);
				Vertices.Add(CornerVert);
				Vertices.Add(CornerVert + FVector(GridSpacing, 0, 0));
				Vertices.Add(CornerVert + FVector(GridSpacing, GridSpacing, 0));
				Vertices.Add(CornerVert + FVector(0, GridSpacing, 0));

				FVector2D UVCorner = FVector2D((float)j / ((float)NumX - 1), (float)i / ((float)NumY - 1));
				UVs.Add(FVector2D(0, 0));
				UVs.Add(FVector2D(1, 0));
				UVs.Add(FVector2D(1, 1));
				UVs.Add(FVector2D(0, 1));

				FVector2D QuadCenter = FVector2D(((float)j + 0.5) / ((float)NumX), ((float)i + 0.5) / ((float)NumY));
				UV1s.Add(QuadCenter);
				UV1s.Add(QuadCenter);
				UV1s.Add(QuadCenter);
				UV1s.Add(QuadCenter);
			}
		}
	}
}


void UKismetProceduralMeshLibrary::GenerateBoxMesh(FVector BoxRadius, TArray<FVector>& Vertices, TArray<int32>& Triangles, TArray<FVector>& Normals, TArray<FVector2D>& UVs, TArray<FProcMeshTangent>& Tangents)
{
	// Generate verts
	FVector BoxVerts[8];
	BoxVerts[0] = FVector(-BoxRadius.X, BoxRadius.Y, BoxRadius.Z);
	BoxVerts[1] = FVector(BoxRadius.X, BoxRadius.Y, BoxRadius.Z);
	BoxVerts[2] = FVector(BoxRadius.X, -BoxRadius.Y, BoxRadius.Z);
	BoxVerts[3] = FVector(-BoxRadius.X, -BoxRadius.Y, BoxRadius.Z);

	BoxVerts[4] = FVector(-BoxRadius.X, BoxRadius.Y, -BoxRadius.Z);
	BoxVerts[5] = FVector(BoxRadius.X, BoxRadius.Y, -BoxRadius.Z);
	BoxVerts[6] = FVector(BoxRadius.X, -BoxRadius.Y, -BoxRadius.Z);
	BoxVerts[7] = FVector(-BoxRadius.X, -BoxRadius.Y, -BoxRadius.Z);

	// Generate triangles (from quads)
	Triangles.Reset();

	const int32 NumVerts = 24; // 6 faces x 4 verts per face

	Vertices.Reset();
	Vertices.AddUninitialized(NumVerts);

	Normals.Reset();
	Normals.AddUninitialized(NumVerts);

	Tangents.Reset();
	Tangents.AddUninitialized(NumVerts);

	Vertices[0] = BoxVerts[0];
	Vertices[1] = BoxVerts[1];
	Vertices[2] = BoxVerts[2];
	Vertices[3] = BoxVerts[3];
	ConvertQuadToTriangles(Triangles, 0, 1, 2, 3);
	Normals[0] = Normals[1] = Normals[2] = Normals[3] = FVector(0, 0, 1);
	Tangents[0] = Tangents[1] = Tangents[2] = Tangents[3] = FProcMeshTangent(0.f, -1.f, 0.f);

	Vertices[4] = BoxVerts[4];
	Vertices[5] = BoxVerts[0];
	Vertices[6] = BoxVerts[3];
	Vertices[7] = BoxVerts[7];
	ConvertQuadToTriangles(Triangles, 4, 5, 6, 7);
	Normals[4] = Normals[5] = Normals[6] = Normals[7] = FVector(-1, 0, 0);
	Tangents[4] = Tangents[5] = Tangents[6] = Tangents[7] = FProcMeshTangent(0.f, -1.f, 0.f);

	Vertices[8] = BoxVerts[5];
	Vertices[9] = BoxVerts[1];
	Vertices[10] = BoxVerts[0];
	Vertices[11] = BoxVerts[4];
	ConvertQuadToTriangles(Triangles, 8, 9, 10, 11);
	Normals[8] = Normals[9] = Normals[10] = Normals[11] = FVector(0, 1, 0);
	Tangents[8] = Tangents[9] = Tangents[10] = Tangents[11] = FProcMeshTangent(-1.f, 0.f, 0.f);

	Vertices[12] = BoxVerts[6];
	Vertices[13] = BoxVerts[2];
	Vertices[14] = BoxVerts[1];
	Vertices[15] = BoxVerts[5];
	ConvertQuadToTriangles(Triangles, 12, 13, 14, 15);
	Normals[12] = Normals[13] = Normals[14] = Normals[15] = FVector(1, 0, 0);
	Tangents[12] = Tangents[13] = Tangents[14] = Tangents[15] = FProcMeshTangent(0.f, 1.f, 0.f);

	Vertices[16] = BoxVerts[7];
	Vertices[17] = BoxVerts[3];
	Vertices[18] = BoxVerts[2];
	Vertices[19] = BoxVerts[6];
	ConvertQuadToTriangles(Triangles, 16, 17, 18, 19);
	Normals[16] = Normals[17] = Normals[18] = Normals[19] = FVector(0, -1, 0);
	Tangents[16] = Tangents[17] = Tangents[18] = Tangents[19] = FProcMeshTangent(1.f, 0.f, 0.f);

	Vertices[20] = BoxVerts[7];
	Vertices[21] = BoxVerts[6];
	Vertices[22] = BoxVerts[5];
	Vertices[23] = BoxVerts[4];
	ConvertQuadToTriangles(Triangles, 20, 21, 22, 23);
	Normals[20] = Normals[21] = Normals[22] = Normals[23] = FVector(0, 0, -1);
	Tangents[20] = Tangents[21] = Tangents[22] = Tangents[23] = FProcMeshTangent(0.f, 1.f, 0.f);

	// UVs
	UVs.Reset();
	UVs.AddUninitialized(NumVerts);

	UVs[0] = UVs[4] = UVs[8] = UVs[12] = UVs[16] = UVs[20] = FVector2D(0.f, 0.f);
	UVs[1] = UVs[5] = UVs[9] = UVs[13] = UVs[17] = UVs[21] = FVector2D(0.f, 1.f);
	UVs[2] = UVs[6] = UVs[10] = UVs[14] = UVs[18] = UVs[22] = FVector2D(1.f, 1.f);
	UVs[3] = UVs[7] = UVs[11] = UVs[15] = UVs[19] = UVs[23] = FVector2D(1.f, 0.f);
}

void FindVertOverlaps(int32 TestVertIndex, const TArray<FVector>& Verts, TArray<int32>& VertOverlaps)
{
	// Check if Verts is empty or test is outside range
	if (TestVertIndex < Verts.Num())
	{
		const FVector TestVert = Verts[TestVertIndex];

		for (int32 VertIdx = 0; VertIdx < Verts.Num(); VertIdx++)
		{
			// First see if we overlap, and smoothing groups are the same
			if (TestVert.Equals(Verts[VertIdx]))
			{
				// If it, so we are at least considered an 'overlap' for normal gen
				VertOverlaps.Add(VertIdx);
			}
		}
	}
}

void UKismetProceduralMeshLibrary::CalculateTangentsForMesh(const TArray<FVector>& Vertices, const TArray<int32>& Triangles, const TArray<FVector2D>& UVs, TArray<FVector>& Normals, TArray<FProcMeshTangent>& Tangents)
{
	SCOPE_CYCLE_COUNTER(STAT_ProcMesh_CalcTangents);

	if (Vertices.Num() == 0)
	{
		return;
	}

	// Number of triangles
	const int32 NumTris = Triangles.Num() / 3;
	// Number of verts
	const int32 NumVerts = Vertices.Num();

	// Map of vertex to triangles in Triangles array
	TMultiMap<int32, int32> VertToTriMap;
	// Map of vertex to triangles to consider for normal calculation
	TMultiMap<int32, int32> VertToTriSmoothMap;

	// Normal/tangents for each face
	TArray<FVector3f> FaceTangentX, FaceTangentY, FaceTangentZ;
	FaceTangentX.AddUninitialized(NumTris);
	FaceTangentY.AddUninitialized(NumTris);
	FaceTangentZ.AddUninitialized(NumTris);

	// Iterate over triangles
	for (int TriIdx = 0; TriIdx < NumTris; TriIdx++)
	{
		int32 CornerIndex[3];
		FVector3f P[3];

		for (int32 CornerIdx = 0; CornerIdx < 3; CornerIdx++)
		{
			// Find vert index (clamped within range)
			int32 VertIndex = FMath::Min(Triangles[(TriIdx * 3) + CornerIdx], NumVerts - 1);

			CornerIndex[CornerIdx] = VertIndex;
			P[CornerIdx] = (FVector3f)Vertices[VertIndex];

			// Find/add this vert to index buffer
			TArray<int32> VertOverlaps;
			FindVertOverlaps(VertIndex, Vertices, VertOverlaps);

			// Remember which triangles map to this vert
			VertToTriMap.AddUnique(VertIndex, TriIdx);
			VertToTriSmoothMap.AddUnique(VertIndex, TriIdx);

			// Also update map of triangles that 'overlap' this vert (ie don't match UV, but do match smoothing) and should be considered when calculating normal
			for (int32 OverlapIdx = 0; OverlapIdx < VertOverlaps.Num(); OverlapIdx++)
			{
				// For each vert we overlap..
				int32 OverlapVertIdx = VertOverlaps[OverlapIdx];

				// Add this triangle to that vert
				VertToTriSmoothMap.AddUnique(OverlapVertIdx, TriIdx);

				// And add all of its triangles to us
				TArray<int32> OverlapTris;
				VertToTriMap.MultiFind(OverlapVertIdx, OverlapTris);
				for (int32 OverlapTriIdx = 0; OverlapTriIdx < OverlapTris.Num(); OverlapTriIdx++)
				{
					VertToTriSmoothMap.AddUnique(VertIndex, OverlapTris[OverlapTriIdx]);
				}
			}
		}

		// Calculate triangle edge vectors and normal
		const FVector3f Edge21 = P[1] - P[2];
		const FVector3f Edge20 = P[0] - P[2];
		const FVector3f TriNormal = (Edge21 ^ Edge20).GetSafeNormal();

		// If we have UVs, use those to calc 
		if (UVs.Num() == Vertices.Num())
		{
			const FVector2D T1 = UVs[CornerIndex[0]];
			const FVector2D T2 = UVs[CornerIndex[1]];
			const FVector2D T3 = UVs[CornerIndex[2]];

			FMatrix	ParameterToLocal(
				FPlane(P[1].X - P[0].X, P[1].Y - P[0].Y, P[1].Z - P[0].Z, 0),
				FPlane(P[2].X - P[0].X, P[2].Y - P[0].Y, P[2].Z - P[0].Z, 0),
				FPlane(P[0].X, P[0].Y, P[0].Z, 0),
				FPlane(0, 0, 0, 1)
				);

			FMatrix ParameterToTexture(
				FPlane(T2.X - T1.X, T2.Y - T1.Y, 0, 0),
				FPlane(T3.X - T1.X, T3.Y - T1.Y, 0, 0),
				FPlane(T1.X, T1.Y, 1, 0),
				FPlane(0, 0, 0, 1)
				);

			// Use InverseSlow to catch singular matrices.  Inverse can miss this sometimes.
			const FMatrix TextureToLocal = ParameterToTexture.Inverse() * ParameterToLocal;

			FaceTangentX[TriIdx] = FVector4f(TextureToLocal.TransformVector(FVector(1, 0, 0)).GetSafeNormal());
			FaceTangentY[TriIdx] = FVector4f(TextureToLocal.TransformVector(FVector(0, 1, 0)).GetSafeNormal());
		}
		else
		{
			FaceTangentX[TriIdx] = Edge20.GetSafeNormal();
			FaceTangentY[TriIdx] = (FaceTangentX[TriIdx] ^ TriNormal).GetSafeNormal();
		}

		FaceTangentZ[TriIdx] = TriNormal;
	}


	// Arrays to accumulate tangents into
	TArray<FVector3f> VertexTangentXSum, VertexTangentYSum, VertexTangentZSum;
	VertexTangentXSum.AddZeroed(NumVerts);
	VertexTangentYSum.AddZeroed(NumVerts);
	VertexTangentZSum.AddZeroed(NumVerts);

	// For each vertex..
	for (int VertxIdx = 0; VertxIdx < Vertices.Num(); VertxIdx++)
	{
		// Find relevant triangles for normal
		TArray<int32> SmoothTris;
		VertToTriSmoothMap.MultiFind(VertxIdx, SmoothTris);

		for (int i = 0; i < SmoothTris.Num(); i++)
		{
			int32 TriIdx = SmoothTris[i];
			VertexTangentZSum[VertxIdx] += FaceTangentZ[TriIdx];
		}

		// Find relevant triangles for tangents
		TArray<int32> TangentTris;
		VertToTriMap.MultiFind(VertxIdx, TangentTris);

		for (int i = 0; i < TangentTris.Num(); i++)
		{
			int32 TriIdx = TangentTris[i];
			VertexTangentXSum[VertxIdx] += FaceTangentX[TriIdx];
			VertexTangentYSum[VertxIdx] += FaceTangentY[TriIdx];
		}
	}

	// Finally, normalize tangents and build output arrays

	Normals.Reset();
	Normals.AddUninitialized(NumVerts);

	Tangents.Reset();
	Tangents.AddUninitialized(NumVerts);

	for (int VertxIdx = 0; VertxIdx < NumVerts; VertxIdx++)
	{
		FVector3f& TangentX = VertexTangentXSum[VertxIdx];
		FVector3f& TangentY = VertexTangentYSum[VertxIdx];
		FVector3f& TangentZ = VertexTangentZSum[VertxIdx];

		TangentX.Normalize();
		TangentZ.Normalize();

		Normals[VertxIdx] = (FVector)TangentZ;

		// Use Gram-Schmidt orthogonalization to make sure X is orth with Z
		TangentX -= TangentZ * (TangentZ | TangentX);
		TangentX.Normalize();

		// See if we need to flip TangentY when generating from cross product
		const bool bFlipBitangent = ((TangentZ ^ TangentX) | TangentY) < 0.f;

		Tangents[VertxIdx] = FProcMeshTangent((FVector)TangentX, bFlipBitangent);
	}
}

static int32 GetNewIndexForOldVertIndex(int32 MeshVertIndex, TMap<int32, int32>& MeshToSectionVertMap, const FStaticMeshVertexBuffers& VertexBuffers, TArray<FVector>& Vertices, TArray<FVector>& Normals, TArray<FVector2D>& UVs, TArray<FProcMeshTangent>& Tangents)
{
	int32* NewIndexPtr = MeshToSectionVertMap.Find(MeshVertIndex);
	if (NewIndexPtr != nullptr)
	{
		return *NewIndexPtr;
	}
	else
	{
		// Copy position
		int32 SectionVertIndex = Vertices.Add((FVector)VertexBuffers.PositionVertexBuffer.VertexPosition(MeshVertIndex));

		// Copy normal
		Normals.Add(FVector4(VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(MeshVertIndex)));
		check(Normals.Num() == Vertices.Num());

		// Copy UVs
		UVs.Add(FVector2D(VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(MeshVertIndex, 0)));
		check(UVs.Num() == Vertices.Num());

		// Copy tangents
		FVector4 TangentX = (FVector4)VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(MeshVertIndex);
		FProcMeshTangent NewTangent(TangentX, TangentX.W < 0.f);
		Tangents.Add(NewTangent);
		check(Tangents.Num() == Vertices.Num());

		MeshToSectionVertMap.Add(MeshVertIndex, SectionVertIndex);

		return SectionVertIndex;
	}
}

void UKismetProceduralMeshLibrary::GetSectionFromStaticMesh(UStaticMesh* InMesh, int32 LODIndex, int32 SectionIndex, TArray<FVector>& Vertices, TArray<int32>& Triangles, TArray<FVector>& Normals, TArray<FVector2D>& UVs, TArray<FProcMeshTangent>& Tangents)
{
	if(	InMesh != nullptr )
	{
		if (!InMesh->bAllowCPUAccess)
		{
			FMessageLog("PIE").Warning()
				->AddToken(FTextToken::Create(LOCTEXT("GetSectionFromStaticMeshStart", "Calling GetSectionFromStaticMesh on")))
				->AddToken(FUObjectToken::Create(InMesh))
				->AddToken(FTextToken::Create(LOCTEXT("GetSectionFromStaticMeshEnd", "but 'Allow CPU Access' is not enabled. This is required for converting StaticMesh to ProceduralMeshComponent in cooked builds.")));
		}

		if (InMesh->GetRenderData() != nullptr && InMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex))
		{
			const FStaticMeshLODResources& LOD = InMesh->GetRenderData()->LODResources[LODIndex];
			if (LOD.Sections.IsValidIndex(SectionIndex))
			{
				// Empty output buffers
				Vertices.Reset();
				Triangles.Reset();
				Normals.Reset();
				UVs.Reset();
				Tangents.Reset();

				// Map from vert buffer for whole mesh to vert buffer for section of interest
				TMap<int32, int32> MeshToSectionVertMap;

				const FStaticMeshSection& Section = LOD.Sections[SectionIndex];
				const uint32 OnePastLastIndex = Section.FirstIndex + Section.NumTriangles * 3;
				FIndexArrayView Indices = LOD.IndexBuffer.GetArrayView();

				// Iterate over section index buffer, copying verts as needed
				for (uint32 i = Section.FirstIndex; i < OnePastLastIndex; i++)
				{
					uint32 MeshVertIndex = Indices[i];

					// See if we have this vert already in our section vert buffer, and copy vert in if not 
					int32 SectionVertIndex = GetNewIndexForOldVertIndex(MeshVertIndex, MeshToSectionVertMap, LOD.VertexBuffers, Vertices, Normals, UVs, Tangents);

					// Add to index buffer
					Triangles.Add(SectionVertIndex);
				}
			}
		}
	}
}

void UKismetProceduralMeshLibrary::CopyProceduralMeshFromStaticMeshComponent(UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, UProceduralMeshComponent* ProcMeshComponent, bool bCreateCollision)
{
	if( StaticMeshComponent != nullptr && 
		StaticMeshComponent->GetStaticMesh() != nullptr &&
		ProcMeshComponent != nullptr )
	{
		UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();

		//// MESH DATA

		int32 NumSections = StaticMesh->GetNumSections(LODIndex);
		for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
		{
			// Buffers for copying geom data
			TArray<FVector> Vertices;
			TArray<int32> Triangles;
			TArray<FVector> Normals;
			TArray<FVector2D> UVs;
			TArray<FVector2D> UVs1;
			TArray<FVector2D> UVs2;
			TArray<FVector2D> UVs3;
			TArray<FProcMeshTangent> Tangents;

			// Get geom data from static mesh
			GetSectionFromStaticMesh(StaticMesh, LODIndex, SectionIndex, Vertices, Triangles, Normals, UVs, Tangents);

			// Create section using data
			TArray<FLinearColor> DummyColors;
			ProcMeshComponent->CreateMeshSection_LinearColor(SectionIndex, Vertices, Triangles, Normals, UVs, UVs1, UVs2, UVs3, DummyColors, Tangents, bCreateCollision);
		}

		//// SIMPLE COLLISION

		// Clear any existing collision hulls
		ProcMeshComponent->ClearCollisionConvexMeshes();

		if (StaticMesh->GetBodySetup() != nullptr)
		{
			// Iterate over all convex hulls on static mesh..
			const int32 NumConvex = StaticMesh->GetBodySetup()->AggGeom.ConvexElems.Num();
			for (int ConvexIndex = 0; ConvexIndex < NumConvex; ConvexIndex++)
			{
				// Copy convex verts to ProcMesh
				FKConvexElem& MeshConvex = StaticMesh->GetBodySetup()->AggGeom.ConvexElems[ConvexIndex];
				ProcMeshComponent->AddCollisionConvexMesh(MeshConvex.VertexData);
			}
		}

		//// MATERIALS

		for (int32 MatIndex = 0; MatIndex < StaticMeshComponent->GetNumMaterials(); MatIndex++)
		{
			ProcMeshComponent->SetMaterial(MatIndex, StaticMeshComponent->GetMaterial(MatIndex));
		}
	}
}

void UKismetProceduralMeshLibrary::GetSectionFromProceduralMesh(UProceduralMeshComponent* InProcMesh, int32 SectionIndex, TArray<FVector>& Vertices, TArray<int32>& Triangles, TArray<FVector>& Normals, TArray<FVector2D>& UVs, TArray<FProcMeshTangent>& Tangents)
{
	if (InProcMesh && SectionIndex >= 0 && SectionIndex < InProcMesh->GetNumSections())
	{
		const FProcMeshSection* Section = InProcMesh->GetProcMeshSection(SectionIndex);

		const int32 NumOutputVerts = Section->ProcVertexBuffer.Num();

		// Allocate output buffers for vert data
		Vertices.SetNumUninitialized(NumOutputVerts);
		Normals.SetNumUninitialized(NumOutputVerts);
		UVs.SetNumUninitialized(NumOutputVerts);
		Tangents.SetNumUninitialized(NumOutputVerts);
		// copy data
		for (int32 VertIdx = 0; VertIdx < NumOutputVerts; VertIdx++)
		{
			const FProcMeshVertex& Vert = Section->ProcVertexBuffer[VertIdx];
			Vertices[VertIdx] = Vert.Position;
			Normals[VertIdx] = Vert.Normal;
			UVs[VertIdx] = Vert.UV0;
			Tangents[VertIdx] = Vert.Tangent;
		}

		// Copy index buffer
		const int32 NumIndices = Section->ProcIndexBuffer.Num();
		Triangles.SetNumUninitialized(NumIndices);
		for (int32 IndexIdx = 0; IndexIdx < NumIndices; IndexIdx++)
		{
			Triangles[IndexIdx] = Section->ProcIndexBuffer[IndexIdx];
		}
	}
}


//////////////////////////////////////////////////////////////////////////

/** Util that returns 1 ir on positive side of plane, -1 if negative, or 0 if split by plane */
int32 BoxPlaneCompare(FBox InBox, const FPlane& InPlane)
{
	FVector BoxCenter, BoxExtents;
	InBox.GetCenterAndExtents(BoxCenter, BoxExtents);

	// Find distance of box center from plane
	FVector::FReal BoxCenterDist = InPlane.PlaneDot(BoxCenter);

	// See size of box in plane normal direction
	FVector::FReal BoxSize = FVector::BoxPushOut(InPlane, BoxExtents);

	if (BoxCenterDist > BoxSize)
	{
		return 1;
	}
	else if (BoxCenterDist < -BoxSize)
	{
		return -1;
	}
	else
	{
		return 0;
	}
}

/** Take two static mesh verts and interpolate all values between them */
FProcMeshVertex InterpolateVert(const FProcMeshVertex& V0, const FProcMeshVertex& V1, float Alpha)
{
	FProcMeshVertex Result;

	// Handle dodgy alpha
	if (FMath::IsNaN(Alpha) || !FMath::IsFinite(Alpha))
	{
		Result = V1;
		return Result;
	}

	Result.Position = FMath::Lerp(V0.Position, V1.Position, Alpha);

	Result.Normal = FMath::Lerp(V0.Normal, V1.Normal, Alpha);

	Result.Tangent.TangentX = FMath::Lerp(V0.Tangent.TangentX, V1.Tangent.TangentX, Alpha);
	Result.Tangent.bFlipTangentY = V0.Tangent.bFlipTangentY; // Assume flipping doesn't change along edge...

	Result.UV0 = FMath::Lerp(V0.UV0, V1.UV0, Alpha);

	Result.Color.R = FMath::Clamp(FMath::TruncToInt(FMath::Lerp(float(V0.Color.R), float(V1.Color.R), Alpha)), 0, 255);
	Result.Color.G = FMath::Clamp(FMath::TruncToInt(FMath::Lerp(float(V0.Color.G), float(V1.Color.G), Alpha)), 0, 255);
	Result.Color.B = FMath::Clamp(FMath::TruncToInt(FMath::Lerp(float(V0.Color.B), float(V1.Color.B), Alpha)), 0, 255);
	Result.Color.A = FMath::Clamp(FMath::TruncToInt(FMath::Lerp(float(V0.Color.A), float(V1.Color.A), Alpha)), 0, 255);

	return Result;
}

/** Transform triangle from 2D to 3D static-mesh triangle. */
void Transform2DPolygonTo3D(const FUtilPoly2D& InPoly, const FMatrix& InMatrix, TArray<FProcMeshVertex>& OutVerts, FBox& OutBox)
{
	FVector3f PolyNormal = (FVector3f)-InMatrix.GetUnitAxis(EAxis::Z);
	FProcMeshTangent PolyTangent(InMatrix.GetUnitAxis(EAxis::X), false);

	for (int32 VertexIndex = 0; VertexIndex < InPoly.Verts.Num(); VertexIndex++)
	{
		const FUtilVertex2D& InVertex = InPoly.Verts[VertexIndex];

		FProcMeshVertex NewVert;

		NewVert.Position = InMatrix.TransformPosition(FVector(InVertex.Pos.X, InVertex.Pos.Y, 0.f));
		NewVert.Normal = (FVector)PolyNormal;
		NewVert.Tangent = PolyTangent;
		NewVert.Color = InVertex.Color;
		NewVert.UV0 = InVertex.UV;

		OutVerts.Add(NewVert);

		// Update bounding box
		OutBox += NewVert.Position;
	}
}

/** Given a polygon, decompose into triangles. */
bool TriangulatePoly(TArray<uint32>& OutTris, const TArray<FProcMeshVertex>& PolyVerts, int32 VertBase, const FVector3f& PolyNormal)
{
	// Can't work if not enough verts for 1 triangle
	int32 NumVerts = PolyVerts.Num() - VertBase;
	if (NumVerts < 3)
	{
		OutTris.Add(0);
		OutTris.Add(2);
		OutTris.Add(1);

		// Return true because poly is already a tri
		return true;
	}

	// Remember initial size of OutTris, in case we need to give up and return to this size
	const int32 TriBase = OutTris.Num();

	// Init array of vert indices, in order. We'll modify this
	TArray<int32> VertIndices;
	VertIndices.AddUninitialized(NumVerts);
	for (int VertIndex = 0; VertIndex < NumVerts; VertIndex++)
	{
		VertIndices[VertIndex] = VertBase + VertIndex;
	}

	// Keep iterating while there are still vertices
	while (VertIndices.Num() >= 3)
	{
		// Look for an 'ear' triangle
		bool bFoundEar = false;
		for (int32 EarVertexIndex = 0; EarVertexIndex < VertIndices.Num(); EarVertexIndex++)
		{
			// Triangle is 'this' vert plus the one before and after it
			const int32 AIndex = (EarVertexIndex == 0) ? VertIndices.Num() - 1 : EarVertexIndex - 1;
			const int32 BIndex = EarVertexIndex;
			const int32 CIndex = (EarVertexIndex + 1) % VertIndices.Num();

			const FProcMeshVertex& AVert = PolyVerts[VertIndices[AIndex]];
			const FProcMeshVertex& BVert = PolyVerts[VertIndices[BIndex]];
			const FProcMeshVertex& CVert = PolyVerts[VertIndices[CIndex]];

			// Check that this vertex is convex (cross product must be positive)
			const FVector3f ABEdge = FVector3f(BVert.Position - AVert.Position);
			const FVector3f ACEdge = FVector3f(CVert.Position - AVert.Position);
			const float TriangleDeterminant = (ABEdge ^ ACEdge) | PolyNormal;
			if (TriangleDeterminant > 0.f)
			{
				continue;
			}

			bool bFoundVertInside = false;
			// Look through all verts before this in array to see if any are inside triangle
			for (int32 VertexIndex = 0; VertexIndex < VertIndices.Num(); VertexIndex++)
			{
				const FProcMeshVertex& TestVert = PolyVerts[VertIndices[VertexIndex]];

				if(	VertexIndex != AIndex && 
					VertexIndex != BIndex && 
					VertexIndex != CIndex &&
					FGeomTools::PointInTriangle((FVector3f)AVert.Position, (FVector3f)BVert.Position, (FVector3f)CVert.Position, (FVector3f)TestVert.Position) )
				{
					bFoundVertInside = true;
					break;
				}
			}

			// Triangle with no verts inside - its an 'ear'! 
			if (!bFoundVertInside)
			{
				OutTris.Add(VertIndices[AIndex]);
				OutTris.Add(VertIndices[CIndex]);
				OutTris.Add(VertIndices[BIndex]);

				// And remove vertex from polygon
				VertIndices.RemoveAt(EarVertexIndex);

				bFoundEar = true;
				break;
			}
		}

		// If we couldn't find an 'ear' it indicates something is bad with this polygon - discard triangles and return.
		if (!bFoundEar)
		{
			OutTris.SetNum(TriBase, EAllowShrinking::Yes);
			return false;
		}
	}

	return true;
}

/** Util to slice a convex hull with a plane */
void SliceConvexElem(const FKConvexElem& InConvex, const FPlane& SlicePlane, TArray<FVector>& OutConvexVerts)
{
	// Get set of planes that make up hull
	TArray<FPlane> ConvexPlanes;
	InConvex.GetPlanes(ConvexPlanes);

	if (ConvexPlanes.Num() >= 4)
	{
		// Add on the slicing plane (need to flip as it culls geom in the opposite sense to our geom culling code)
		ConvexPlanes.Add(SlicePlane.Flip());

		// Create output convex based on new set of planes
		FKConvexElem SlicedElem;
		bool bSuccess = SlicedElem.HullFromPlanes(ConvexPlanes, InConvex.VertexData);
		if (bSuccess)
		{
			OutConvexVerts = SlicedElem.VertexData;
		}
	}
}

void UKismetProceduralMeshLibrary::SliceProceduralMesh(UProceduralMeshComponent* InProcMesh, FVector PlanePosition, FVector PlaneNormal, bool bCreateOtherHalf, UProceduralMeshComponent*& OutOtherHalfProcMesh, EProcMeshSliceCapOption CapOption, UMaterialInterface* CapMaterial)
{
	if (InProcMesh != nullptr)
	{
		// Transform plane from world to local space
		FTransform ProcCompToWorld = InProcMesh->GetComponentToWorld();
		FVector LocalPlanePos = ProcCompToWorld.InverseTransformPosition(PlanePosition);
		FVector LocalPlaneNormal = ProcCompToWorld.GetScale3D() * ProcCompToWorld.InverseTransformVectorNoScale(PlaneNormal);
		LocalPlaneNormal = LocalPlaneNormal.GetSafeNormal(); // Ensure normalized

		FPlane SlicePlane(LocalPlanePos, LocalPlaneNormal);

		// Set of sections to add to the 'other half' component
		TArray<FProcMeshSection> OtherSections;
		// Material for each section of other half
		TArray<UMaterialInterface*> OtherMaterials;

		// Set of new edges created by clipping polys by plane
		TArray<FUtilEdge3D> ClipEdges;

		for (int32 SectionIndex = 0; SectionIndex < InProcMesh->GetNumSections(); SectionIndex++)
		{
			FProcMeshSection* BaseSection = InProcMesh->GetProcMeshSection(SectionIndex);
			// If we have a section, and it has some valid geom
			if (BaseSection != nullptr && BaseSection->ProcIndexBuffer.Num() > 0 && BaseSection->ProcVertexBuffer.Num() > 0)
			{
				// Compare bounding box of section with slicing plane
				int32 BoxCompare = BoxPlaneCompare(BaseSection->SectionLocalBox, SlicePlane);

				// Box totally clipped, clear section
				if (BoxCompare == -1)
				{
					// Add entire section to other half
					if (bCreateOtherHalf)
					{
						OtherSections.Add(*BaseSection);
						OtherMaterials.Add(InProcMesh->GetMaterial(SectionIndex));
					}

					InProcMesh->ClearMeshSection(SectionIndex);
				}
				// Box totally on one side of plane, leave it alone, do nothing
				else if (BoxCompare == 1)
				{
					// ...
				}
				// Box intersects plane, need to clip some polys!
				else
				{
					// New section for geometry
					FProcMeshSection NewSection;

					// New section for 'other half' geometry (if desired)
					FProcMeshSection* NewOtherSection = nullptr;
					if (bCreateOtherHalf)
					{
						int32 OtherSectionIndex = OtherSections.Add(FProcMeshSection());
						NewOtherSection = &OtherSections[OtherSectionIndex];

						OtherMaterials.Add(InProcMesh->GetMaterial(SectionIndex)); // Remember material for this section
					}

					// Map of base vert index to sliced vert index
					TMap<int32, int32> BaseToSlicedVertIndex;
					TMap<int32, int32> BaseToOtherSlicedVertIndex;

					const int32 NumBaseVerts = BaseSection->ProcVertexBuffer.Num();

					// Distance of each base vert from slice plane
					TArray<float> VertDistance;
					VertDistance.AddUninitialized(NumBaseVerts);

					// Build vertex buffer 
					for (int32 BaseVertIndex = 0; BaseVertIndex < NumBaseVerts; BaseVertIndex++)
					{
						FProcMeshVertex& BaseVert = BaseSection->ProcVertexBuffer[BaseVertIndex];

						// Calc distance from plane
						VertDistance[BaseVertIndex] = SlicePlane.PlaneDot(BaseVert.Position);

						// See if vert is being kept in this section
						if (VertDistance[BaseVertIndex] > 0.f)
						{
							// Copy to sliced v buffer
							int32 SlicedVertIndex = NewSection.ProcVertexBuffer.Add(BaseVert);
							// Update section bounds
							NewSection.SectionLocalBox += BaseVert.Position;
							// Add to map
							BaseToSlicedVertIndex.Add(BaseVertIndex, SlicedVertIndex);
						}
						// Or add to other half if desired
						else if(NewOtherSection != nullptr)
						{
							int32 SlicedVertIndex = NewOtherSection->ProcVertexBuffer.Add(BaseVert);
							NewOtherSection->SectionLocalBox += BaseVert.Position;
							BaseToOtherSlicedVertIndex.Add(BaseVertIndex, SlicedVertIndex);
						}
					}


					// Iterate over base triangles (ie 3 indices at a time)
					for (int32 BaseIndex = 0; BaseIndex < BaseSection->ProcIndexBuffer.Num(); BaseIndex += 3)
					{
						int32 BaseV[3]; // Triangle vert indices in original mesh
						int32* SlicedV[3]; // Pointers to vert indices in new v buffer
						int32* SlicedOtherV[3]; // Pointers to vert indices in new 'other half' v buffer

						// For each vertex..
						for (int32 i = 0; i < 3; i++)
						{
							// Get triangle vert index
							BaseV[i] = BaseSection->ProcIndexBuffer[BaseIndex + i];
							// Look up in sliced v buffer
							SlicedV[i] = BaseToSlicedVertIndex.Find(BaseV[i]);
							// Look up in 'other half' v buffer (if desired)
							if (bCreateOtherHalf)
							{
								SlicedOtherV[i] = BaseToOtherSlicedVertIndex.Find(BaseV[i]);
								// Each base vert _must_ exist in either BaseToSlicedVertIndex or BaseToOtherSlicedVertIndex 
								check((SlicedV[i] != nullptr) != (SlicedOtherV[i] != nullptr));
							}
						}

						// If all verts survived plane cull, keep the triangle
						if (SlicedV[0] != nullptr && SlicedV[1] != nullptr && SlicedV[2] != nullptr)
						{
							NewSection.ProcIndexBuffer.Add(*SlicedV[0]);
							NewSection.ProcIndexBuffer.Add(*SlicedV[1]);
							NewSection.ProcIndexBuffer.Add(*SlicedV[2]);
						}
						// If all verts were removed by plane cull
						else if (SlicedV[0] == nullptr && SlicedV[1] == nullptr && SlicedV[2] == nullptr)
						{
							// If creating other half, add all verts to that
							if (NewOtherSection != nullptr)
							{
								NewOtherSection->ProcIndexBuffer.Add(*SlicedOtherV[0]);
								NewOtherSection->ProcIndexBuffer.Add(*SlicedOtherV[1]);
								NewOtherSection->ProcIndexBuffer.Add(*SlicedOtherV[2]);
							}
						}
						// If partially culled, clip to create 1 or 2 new triangles
						else
						{
							int32 FinalVerts[4];
							int32 NumFinalVerts = 0;

							int32 OtherFinalVerts[4];
							int32 NumOtherFinalVerts = 0;

							FUtilEdge3D NewClipEdge;
							int32 ClippedEdges = 0;

							float PlaneDist[3];
							PlaneDist[0] = VertDistance[BaseV[0]];
							PlaneDist[1] = VertDistance[BaseV[1]];
							PlaneDist[2] = VertDistance[BaseV[2]];

							for (int32 EdgeIdx = 0; EdgeIdx < 3; EdgeIdx++)
							{
								int32 ThisVert = EdgeIdx;

								// If start vert is inside, add it.
								if (SlicedV[ThisVert] != nullptr)
								{
									check(NumFinalVerts < 4);
									FinalVerts[NumFinalVerts++] = *SlicedV[ThisVert];
								}
								// If not, add to other side
								else if(bCreateOtherHalf)
								{
									check(NumOtherFinalVerts < 4);
									OtherFinalVerts[NumOtherFinalVerts++] = *SlicedOtherV[ThisVert];
								}

								// If start and next vert are on opposite sides, add intersection
								int32 NextVert = (EdgeIdx + 1) % 3;

								if ((SlicedV[EdgeIdx] == nullptr) != (SlicedV[NextVert] == nullptr))
								{
									// Find distance along edge that plane is
									float Alpha = -PlaneDist[ThisVert] / (PlaneDist[NextVert] - PlaneDist[ThisVert]);
									// Interpolate vertex params to that point
									FProcMeshVertex InterpVert = InterpolateVert(BaseSection->ProcVertexBuffer[BaseV[ThisVert]], BaseSection->ProcVertexBuffer[BaseV[NextVert]], FMath::Clamp(Alpha, 0.0f, 1.0f));

									// Add to vertex buffer
									int32 InterpVertIndex = NewSection.ProcVertexBuffer.Add(InterpVert);
									// Update bounds
									NewSection.SectionLocalBox += InterpVert.Position;

									// Save vert index for this poly
									check(NumFinalVerts < 4);
									FinalVerts[NumFinalVerts++] = InterpVertIndex;

									// If desired, add to the poly for the other half as well
									if (NewOtherSection != nullptr)
									{
										int32 OtherInterpVertIndex = NewOtherSection->ProcVertexBuffer.Add(InterpVert);
										NewOtherSection->SectionLocalBox += InterpVert.Position;
										check(NumOtherFinalVerts < 4);
										OtherFinalVerts[NumOtherFinalVerts++] = OtherInterpVertIndex;
									}

									// When we make a new edge on the surface of the clip plane, save it off.
									check(ClippedEdges < 2);
									if (ClippedEdges == 0)
									{
										NewClipEdge.V0 = (FVector3f)InterpVert.Position;
									}
									else
									{
										NewClipEdge.V1 = (FVector3f)InterpVert.Position;
									}

									ClippedEdges++;
								}
							}

							// Triangulate the clipped polygon.
							for (int32 VertexIndex = 2; VertexIndex < NumFinalVerts; VertexIndex++)
							{
								NewSection.ProcIndexBuffer.Add(FinalVerts[0]);
								NewSection.ProcIndexBuffer.Add(FinalVerts[VertexIndex - 1]);
								NewSection.ProcIndexBuffer.Add(FinalVerts[VertexIndex]);
							}

							// If we are making the other half, triangulate that as well
							if (NewOtherSection != nullptr)
							{
								for (int32 VertexIndex = 2; VertexIndex < NumOtherFinalVerts; VertexIndex++)
								{
									NewOtherSection->ProcIndexBuffer.Add(OtherFinalVerts[0]);
									NewOtherSection->ProcIndexBuffer.Add(OtherFinalVerts[VertexIndex - 1]);
									NewOtherSection->ProcIndexBuffer.Add(OtherFinalVerts[VertexIndex]);
								}
							}

							check(ClippedEdges != 1); // Should never clip just one edge of the triangle

							// If we created a new edge, save that off here as well
							if (ClippedEdges == 2)
							{
								ClipEdges.Add(NewClipEdge);
							}
						}
					}

					// Remove 'other' section from array if no valid geometry for it
					if (NewOtherSection != nullptr && (NewOtherSection->ProcIndexBuffer.Num() == 0 || NewOtherSection->ProcVertexBuffer.Num() == 0))
					{
						OtherSections.RemoveAt(OtherSections.Num() - 1);
					}

					// If we have some valid geometry, update section
					if (NewSection.ProcIndexBuffer.Num() > 0 && NewSection.ProcVertexBuffer.Num() > 0)
					{
						// Assign new geom to this section
						InProcMesh->SetProcMeshSection(SectionIndex, NewSection);
					}
					// If we don't, remove this section
					else
					{
						InProcMesh->ClearMeshSection(SectionIndex);
					}
				}
			}
		}

		// Create cap geometry (if some edges to create it from)
		if (CapOption != EProcMeshSliceCapOption::NoCap && ClipEdges.Num() > 0)
		{
			FProcMeshSection CapSection;
			int32 CapSectionIndex = INDEX_NONE;

			// If using an existing section, copy that info first
			if (CapOption == EProcMeshSliceCapOption::UseLastSectionForCap)
			{
				CapSectionIndex = InProcMesh->GetNumSections() - 1;
				CapSection = *InProcMesh->GetProcMeshSection(CapSectionIndex);
			}
			// Adding new section for cap
			else
			{
				CapSectionIndex = InProcMesh->GetNumSections();
			}

			// Project 3D edges onto slice plane to form 2D edges
			TArray<FUtilEdge2D> Edges2D;
			FUtilPoly2DSet PolySet;
			FGeomTools::ProjectEdges(Edges2D, PolySet.PolyToWorld, ClipEdges, SlicePlane);

			// Find 2D closed polygons from this edge soup
			FGeomTools::Buid2DPolysFromEdges(PolySet.Polys, Edges2D, FColor(255, 255, 255, 255));

			// Remember start point for vert and index buffer before adding and cap geom
			int32 CapVertBase = CapSection.ProcVertexBuffer.Num();
			int32 CapIndexBase = CapSection.ProcIndexBuffer.Num();

			// Triangulate each poly
			for (int32 PolyIdx = 0; PolyIdx < PolySet.Polys.Num(); PolyIdx++)
			{
				// Generate UVs for the 2D polygon.
				FGeomTools::GeneratePlanarTilingPolyUVs(PolySet.Polys[PolyIdx], 64.f);

				// Remember start of vert buffer before adding triangles for this poly
				int32 PolyVertBase = CapSection.ProcVertexBuffer.Num();

				// Transform from 2D poly verts to 3D
				Transform2DPolygonTo3D(PolySet.Polys[PolyIdx], PolySet.PolyToWorld, CapSection.ProcVertexBuffer, CapSection.SectionLocalBox);

				// Triangulate this polygon
				TriangulatePoly(CapSection.ProcIndexBuffer, CapSection.ProcVertexBuffer, PolyVertBase, (FVector3f)LocalPlaneNormal);
			}

			// Set geom for cap section
			InProcMesh->SetProcMeshSection(CapSectionIndex, CapSection);

			// If creating new section for cap, assign cap material to it
			if (CapOption == EProcMeshSliceCapOption::CreateNewSectionForCap)
			{
				InProcMesh->SetMaterial(CapSectionIndex, CapMaterial);
			}

			// If creating the other half, copy cap geom into other half sections
			if (bCreateOtherHalf)
			{
				// Find section we want to use for the cap on the 'other half'
				FProcMeshSection* OtherCapSection;
				if (CapOption == EProcMeshSliceCapOption::CreateNewSectionForCap)
				{
					OtherSections.Add(FProcMeshSection());
					OtherMaterials.Add(CapMaterial);
				}
				OtherCapSection = &OtherSections.Last();

				// Remember current base index for verts in 'other cap section'
				int32 OtherCapVertBase = OtherCapSection->ProcVertexBuffer.Num();

				// Copy verts from cap section into other cap section
				for (int32 VertIdx = CapVertBase; VertIdx < CapSection.ProcVertexBuffer.Num(); VertIdx++)
				{
					FProcMeshVertex OtherCapVert = CapSection.ProcVertexBuffer[VertIdx];

					// Flip normal and tangent TODO: FlipY?
					OtherCapVert.Normal *= -1.f;
					OtherCapVert.Tangent.TangentX *= -1.f;

					// Add to other cap v buffer
					OtherCapSection->ProcVertexBuffer.Add(OtherCapVert);
					// And update bounding box
					OtherCapSection->SectionLocalBox += OtherCapVert.Position;
				}

				// Find offset between main cap verts and other cap verts
				int32 VertOffset = OtherCapVertBase - CapVertBase;

				// Copy indices over as well
				for (int32 IndexIdx = CapIndexBase; IndexIdx < CapSection.ProcIndexBuffer.Num(); IndexIdx += 3)
				{
					// Need to offset and change winding
					OtherCapSection->ProcIndexBuffer.Add(CapSection.ProcIndexBuffer[IndexIdx + 0] + VertOffset);
					OtherCapSection->ProcIndexBuffer.Add(CapSection.ProcIndexBuffer[IndexIdx + 2] + VertOffset);
					OtherCapSection->ProcIndexBuffer.Add(CapSection.ProcIndexBuffer[IndexIdx + 1] + VertOffset);
				}
			}
		}

		// Array of sliced collision shapes
		TArray< TArray<FVector> > SlicedCollision;
		TArray< TArray<FVector> > OtherSlicedCollision;

		UBodySetup* ProcMeshBodySetup = InProcMesh->GetBodySetup();

		for (int32 ConvexIndex = 0; ConvexIndex < ProcMeshBodySetup->AggGeom.ConvexElems.Num(); ConvexIndex++)
		{
			FKConvexElem& BaseConvex = ProcMeshBodySetup->AggGeom.ConvexElems[ConvexIndex];

			int32 BoxCompare = BoxPlaneCompare(BaseConvex.ElemBox, SlicePlane);

			// If box totally clipped, add to other half (if desired)
			if (BoxCompare == -1)
			{
				if (bCreateOtherHalf)
				{
					OtherSlicedCollision.Add(BaseConvex.VertexData);
				}
			}
			// If box totally valid, just keep mesh as is
			else if (BoxCompare == 1)
			{
				SlicedCollision.Add(BaseConvex.VertexData);				// LWC_TODO: Perf pessimization
			}
			// Need to actually slice the convex shape
			else
			{
				TArray<FVector> SlicedConvexVerts;
				SliceConvexElem(BaseConvex, SlicePlane, SlicedConvexVerts);
				// If we got something valid, add it
				if (SlicedConvexVerts.Num() >= 4)
				{
					SlicedCollision.Add(SlicedConvexVerts);
				}

				// Slice again to get the other half of the collision, if desired
				if (bCreateOtherHalf)
				{
					TArray<FVector> OtherSlicedConvexVerts;
					SliceConvexElem(BaseConvex, SlicePlane.Flip(), OtherSlicedConvexVerts);
					if (OtherSlicedConvexVerts.Num() >= 4)
					{
						OtherSlicedCollision.Add(OtherSlicedConvexVerts);
					}
				}
			}
		}

		// Update collision of proc mesh
		InProcMesh->SetCollisionConvexMeshes(SlicedCollision);

		// If creating other half, create component now
		if (bCreateOtherHalf)
		{
			// Create new component with the same outer as the proc mesh passed in
			OutOtherHalfProcMesh = NewObject<UProceduralMeshComponent>(InProcMesh->GetOuter());

			// Set transform to match source component
			OutOtherHalfProcMesh->SetWorldTransform(InProcMesh->GetComponentTransform());

			// Add each section of geometry
			for (int32 SectionIndex = 0; SectionIndex < OtherSections.Num(); SectionIndex++)
			{
				OutOtherHalfProcMesh->SetProcMeshSection(SectionIndex, OtherSections[SectionIndex]);
				OutOtherHalfProcMesh->SetMaterial(SectionIndex, OtherMaterials[SectionIndex]);
			}

			// Copy collision settings from input mesh
			OutOtherHalfProcMesh->SetCollisionProfileName(InProcMesh->GetCollisionProfileName());
			OutOtherHalfProcMesh->SetCollisionEnabled(InProcMesh->GetCollisionEnabled());
			OutOtherHalfProcMesh->bUseComplexAsSimpleCollision = InProcMesh->bUseComplexAsSimpleCollision;

			// Assign sliced collision
			OutOtherHalfProcMesh->SetCollisionConvexMeshes(OtherSlicedCollision);

			// Finally register
			OutOtherHalfProcMesh->RegisterComponent();
		}
	}
}

#undef LOCTEXT_NAMESPACE

