// Copyright Epic Games, Inc. All Rights Reserved. 

#include "ProceduralMeshComponent.h"
#include "BodySetupEnums.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "Engine/Engine.h"
#include "RenderUtils.h"
#include "PrimitiveDrawingUtils.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "PhysicsEngine/BodySetup.h"
#include "ProceduralMeshComponentPluginPrivate.h"
#include "DynamicMeshBuilder.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "StaticMeshResources.h"
#include "RayTracingInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralMeshComponent)

DECLARE_CYCLE_STAT(TEXT("Create ProcMesh Proxy"), STAT_ProcMesh_CreateSceneProxy, STATGROUP_ProceduralMesh);
DECLARE_CYCLE_STAT(TEXT("Create Mesh Section"), STAT_ProcMesh_CreateMeshSection, STATGROUP_ProceduralMesh);
DECLARE_CYCLE_STAT(TEXT("UpdateSection GT"), STAT_ProcMesh_UpdateSectionGT, STATGROUP_ProceduralMesh);
DECLARE_CYCLE_STAT(TEXT("UpdateSection RT"), STAT_ProcMesh_UpdateSectionRT, STATGROUP_ProceduralMesh);
DECLARE_CYCLE_STAT(TEXT("Get ProcMesh Elements"), STAT_ProcMesh_GetMeshElements, STATGROUP_ProceduralMesh);
DECLARE_CYCLE_STAT(TEXT("Update Collision"), STAT_ProcMesh_UpdateCollision, STATGROUP_ProceduralMesh);

DEFINE_LOG_CATEGORY_STATIC(LogProceduralComponent, Log, All);

static TAutoConsoleVariable<int32> CVarRayTracingProceduralMesh(
	TEXT("r.RayTracing.Geometry.ProceduralMeshes"),
	1,
	TEXT("Include procedural meshes in ray tracing effects (default = 1 (procedural meshes enabled in ray tracing))"));

/** Class representing a single section of the proc mesh */
class FProcMeshProxySection
{
public:
	/** Material applied to this section */
	UMaterialInterface* Material;
	/** Vertex buffer for this section */
	FStaticMeshVertexBuffers VertexBuffers;
	/** Index buffer for this section */
	FDynamicMeshIndexBuffer32 IndexBuffer;
	/** Vertex factory for this section */
	FLocalVertexFactory VertexFactory;
	/** Whether this section is currently visible */
	bool bSectionVisible;

#if RHI_RAYTRACING
	FRayTracingGeometry RayTracingGeometry;
#endif

	FProcMeshProxySection(ERHIFeatureLevel::Type InFeatureLevel)
	: Material(NULL)
	, VertexFactory(InFeatureLevel, "FProcMeshProxySection")
	, bSectionVisible(true)
	{}
};

/** 
 *	Struct used to send update to mesh data 
 *	Arrays may be empty, in which case no update is performed.
 */
class FProcMeshSectionUpdateData
{
public:
	/** Section to update */
	int32 TargetSection;
	/** New vertex information */
	TArray<FProcMeshVertex> NewVertexBuffer;
};

static void ConvertProcMeshToDynMeshVertex(FDynamicMeshVertex& Vert, const FProcMeshVertex& ProcVert)
{
	Vert.Position = (FVector3f)ProcVert.Position;
	Vert.Color = ProcVert.Color;
	Vert.TextureCoordinate[0] = FVector2f(ProcVert.UV0);	// LWC_TODO: Precision loss
	Vert.TextureCoordinate[1] = FVector2f(ProcVert.UV1);	// LWC_TODO: Precision loss
	Vert.TextureCoordinate[2] = FVector2f(ProcVert.UV2);	// LWC_TODO: Precision loss
	Vert.TextureCoordinate[3] = FVector2f(ProcVert.UV3);	// LWC_TODO: Precision loss
	Vert.TangentX = ProcVert.Tangent.TangentX;
	Vert.TangentZ = ProcVert.Normal;
	Vert.TangentZ.Vector.W = ProcVert.Tangent.bFlipTangentY ? -127 : 127;
}

/** Procedural mesh scene proxy */
class FProceduralMeshSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FProceduralMeshSceneProxy(UProceduralMeshComponent* Component)
		: FPrimitiveSceneProxy(Component)
		, BodySetup(Component->GetBodySetup())
		, MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform()))
	{
		// Copy each section
		const int32 NumSections = Component->ProcMeshSections.Num();
		Sections.AddZeroed(NumSections);
		for (int SectionIdx = 0; SectionIdx < NumSections; SectionIdx++)
		{
			FProcMeshSection& SrcSection = Component->ProcMeshSections[SectionIdx];
			if (SrcSection.ProcIndexBuffer.Num() > 0 && SrcSection.ProcVertexBuffer.Num() > 0)
			{
				FProcMeshProxySection* NewSection = new FProcMeshProxySection(GetScene().GetFeatureLevel());

				// Copy data from vertex buffer
				const int32 NumVerts = SrcSection.ProcVertexBuffer.Num();

				// Allocate verts

				TArray<FDynamicMeshVertex> Vertices;
				Vertices.SetNumUninitialized(NumVerts);
				// Copy verts
				for (int VertIdx = 0; VertIdx < NumVerts; VertIdx++)
				{
					const FProcMeshVertex& ProcVert = SrcSection.ProcVertexBuffer[VertIdx];
					FDynamicMeshVertex& Vert = Vertices[VertIdx];
					ConvertProcMeshToDynMeshVertex(Vert, ProcVert);
				}

				// Copy index buffer
				NewSection->IndexBuffer.Indices = SrcSection.ProcIndexBuffer;

				NewSection->VertexBuffers.InitFromDynamicVertex(&NewSection->VertexFactory, Vertices, 4);

				// Enqueue initialization of render resource
				BeginInitResource(&NewSection->VertexBuffers.PositionVertexBuffer);
				BeginInitResource(&NewSection->VertexBuffers.StaticMeshVertexBuffer);
				BeginInitResource(&NewSection->VertexBuffers.ColorVertexBuffer);
				BeginInitResource(&NewSection->IndexBuffer);
				BeginInitResource(&NewSection->VertexFactory);

				// Grab material
				NewSection->Material = Component->GetMaterial(SectionIdx);
				if (NewSection->Material == NULL)
				{
					NewSection->Material = UMaterial::GetDefaultMaterial(MD_Surface);
				}

				// Copy visibility info
				NewSection->bSectionVisible = SrcSection.bSectionVisible;

				// Save ref to new section
				Sections[SectionIdx] = NewSection;

#if RHI_RAYTRACING
				if (IsRayTracingEnabled())
				{
					ENQUEUE_RENDER_COMMAND(InitProceduralMeshRayTracingGeometry)(
						[this, DebugName = Component->GetFName(), NewSection](FRHICommandListImmediate& RHICmdList)
					{
						FRayTracingGeometryInitializer Initializer;
						Initializer.DebugName = DebugName;
						Initializer.IndexBuffer = NewSection->IndexBuffer.IndexBufferRHI;
						Initializer.TotalPrimitiveCount = NewSection->IndexBuffer.Indices.Num() / 3;
						Initializer.GeometryType = RTGT_Triangles;
						Initializer.bFastBuild = true;
						Initializer.bAllowUpdate = false;

						FRayTracingGeometrySegment Segment;
						Segment.VertexBuffer = NewSection->VertexBuffers.PositionVertexBuffer.VertexBufferRHI;
						Segment.MaxVertices = NewSection->VertexBuffers.PositionVertexBuffer.GetNumVertices();
						Segment.NumPrimitives = Initializer.TotalPrimitiveCount;
						
						Initializer.Segments.Add(Segment);

						NewSection->RayTracingGeometry.SetInitializer(Initializer);
						NewSection->RayTracingGeometry.InitResource(RHICmdList);
					});
				}
#endif
			}
		}
	}

	virtual ~FProceduralMeshSceneProxy()
	{
		for (FProcMeshProxySection* Section : Sections)
		{
			if (Section != nullptr)
			{
				Section->VertexBuffers.PositionVertexBuffer.ReleaseResource();
				Section->VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
				Section->VertexBuffers.ColorVertexBuffer.ReleaseResource();
				Section->IndexBuffer.ReleaseResource();
				Section->VertexFactory.ReleaseResource();

#if RHI_RAYTRACING
				Section->RayTracingGeometry.ReleaseResource();
#endif

				delete Section;
			}
		}
	}

	/** Called on render thread to assign new dynamic data */
	void UpdateSection_RenderThread(FRHICommandListBase& RHICmdList, FProcMeshSectionUpdateData* SectionData)
	{
		SCOPE_CYCLE_COUNTER(STAT_ProcMesh_UpdateSectionRT);

		// Check we have data 
		if(	SectionData != nullptr) 			
		{
			// Check it references a valid section
			if (SectionData->TargetSection < Sections.Num() &&
				Sections[SectionData->TargetSection] != nullptr)
			{
				FProcMeshProxySection* Section = Sections[SectionData->TargetSection];

				// Lock vertex buffer
				const int32 NumVerts = SectionData->NewVertexBuffer.Num();
			
				// Iterate through vertex data, copying in new info
				for(int32 i=0; i<NumVerts; i++)
				{
					const FProcMeshVertex& ProcVert = SectionData->NewVertexBuffer[i];
					FDynamicMeshVertex Vertex;
					ConvertProcMeshToDynMeshVertex(Vertex, ProcVert);

					Section->VertexBuffers.PositionVertexBuffer.VertexPosition(i) = Vertex.Position;
					Section->VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(i, Vertex.TangentX.ToFVector3f(), Vertex.GetTangentY(), Vertex.TangentZ.ToFVector3f());
					Section->VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(i, 0, Vertex.TextureCoordinate[0]);
					Section->VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(i, 1, Vertex.TextureCoordinate[1]);
					Section->VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(i, 2, Vertex.TextureCoordinate[2]);
					Section->VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(i, 3, Vertex.TextureCoordinate[3]);
					Section->VertexBuffers.ColorVertexBuffer.VertexColor(i) = Vertex.Color;
				}

				{
					auto& VertexBuffer = Section->VertexBuffers.PositionVertexBuffer;
					void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetNumVertices() * VertexBuffer.GetStride(), RLM_WriteOnly);
					FMemory::Memcpy(VertexBufferData, VertexBuffer.GetVertexData(), VertexBuffer.GetNumVertices() * VertexBuffer.GetStride());
					RHICmdList.UnlockBuffer(VertexBuffer.VertexBufferRHI);
				}

				{
					auto& VertexBuffer = Section->VertexBuffers.ColorVertexBuffer;
					void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetNumVertices() * VertexBuffer.GetStride(), RLM_WriteOnly);
					FMemory::Memcpy(VertexBufferData, VertexBuffer.GetVertexData(), VertexBuffer.GetNumVertices() * VertexBuffer.GetStride());
					RHICmdList.UnlockBuffer(VertexBuffer.VertexBufferRHI);
				}

				{
					auto& VertexBuffer = Section->VertexBuffers.StaticMeshVertexBuffer;
					void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.TangentsVertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetTangentSize(), RLM_WriteOnly);
					FMemory::Memcpy(VertexBufferData, VertexBuffer.GetTangentData(), VertexBuffer.GetTangentSize());
					RHICmdList.UnlockBuffer(VertexBuffer.TangentsVertexBuffer.VertexBufferRHI);
				}

				{
					auto& VertexBuffer = Section->VertexBuffers.StaticMeshVertexBuffer;
					void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.TexCoordVertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetTexCoordSize(), RLM_WriteOnly);
					FMemory::Memcpy(VertexBufferData, VertexBuffer.GetTexCoordData(), VertexBuffer.GetTexCoordSize());
					RHICmdList.UnlockBuffer(VertexBuffer.TexCoordVertexBuffer.VertexBufferRHI);
				}

#if RHI_RAYTRACING
				if (IsRayTracingEnabled())
				{
					Section->RayTracingGeometry.ReleaseResource();

					FRayTracingGeometryInitializer Initializer;
					Initializer.IndexBuffer = Section->IndexBuffer.IndexBufferRHI;
					Initializer.TotalPrimitiveCount = Section->IndexBuffer.Indices.Num() / 3;
					Initializer.GeometryType = RTGT_Triangles;
					Initializer.bFastBuild = true;
					Initializer.bAllowUpdate = false;

					FRayTracingGeometrySegment Segment;
					Segment.VertexBuffer = Section->VertexBuffers.PositionVertexBuffer.VertexBufferRHI;
					Segment.MaxVertices = Section->VertexBuffers.PositionVertexBuffer.GetNumVertices();
					Segment.NumPrimitives = Initializer.TotalPrimitiveCount;

					Initializer.Segments.Add(Segment);

					Section->RayTracingGeometry.SetInitializer(Initializer);
					Section->RayTracingGeometry.InitResource(RHICmdList);
				}
#endif
			}

			// Free data sent from game thread
			delete SectionData;
		}
	}

	void SetSectionVisibility_RenderThread(int32 SectionIndex, bool bNewVisibility)
	{
		check(IsInRenderingThread());

		if(	SectionIndex < Sections.Num() &&
			Sections[SectionIndex] != nullptr)
		{
			Sections[SectionIndex]->bSectionVisible = bNewVisibility;
		}
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		SCOPE_CYCLE_COUNTER(STAT_ProcMesh_GetMeshElements);


		// Set up wireframe material (if needed)
		const bool bWireframe = AllowDebugViewmodes() && ViewFamily.EngineShowFlags.Wireframe;

		FColoredMaterialRenderProxy* WireframeMaterialInstance = NULL;
		if (bWireframe)
		{
			WireframeMaterialInstance = new FColoredMaterialRenderProxy(
				GEngine->WireframeMaterial ? GEngine->WireframeMaterial->GetRenderProxy() : NULL,
				FLinearColor(0, 0.5f, 1.f)
				);

			Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);
		}

		// Iterate over sections
		for (const FProcMeshProxySection* Section : Sections)
		{
			if (Section != nullptr && Section->bSectionVisible)
			{
				FMaterialRenderProxy* MaterialProxy = bWireframe ? WireframeMaterialInstance : Section->Material->GetRenderProxy();

				// For each view..
				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					if (VisibilityMap & (1 << ViewIndex))
					{
						const FSceneView* View = Views[ViewIndex];
						// Draw the mesh.
						FMeshBatch& Mesh = Collector.AllocateMesh();
						FMeshBatchElement& BatchElement = Mesh.Elements[0];
						BatchElement.IndexBuffer = &Section->IndexBuffer;
						Mesh.bWireframe = bWireframe;
						Mesh.VertexFactory = &Section->VertexFactory;
						Mesh.MaterialRenderProxy = MaterialProxy;

						FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
						FPrimitiveUniformShaderParametersBuilder Builder;
						BuildUniformShaderParameters(Builder);
						DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);

						BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

						BatchElement.FirstIndex = 0;
						BatchElement.NumPrimitives = Section->IndexBuffer.Indices.Num() / 3;
						BatchElement.MinVertexIndex = 0;
						BatchElement.MaxVertexIndex = Section->VertexBuffers.PositionVertexBuffer.GetNumVertices() - 1;
						Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
						Mesh.Type = PT_TriangleList;
						Mesh.DepthPriorityGroup = SDPG_World;
						Mesh.bCanApplyViewModeOverrides = false;
						Collector.AddMesh(ViewIndex, Mesh);
					}
				}
			}
		}

		// Draw bounds
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{	 
				// Draw simple collision as wireframe if 'show collision', and collision is enabled, and we are not using the complex as the simple
				if (ViewFamily.EngineShowFlags.Collision && IsCollisionEnabled() && BodySetup->GetCollisionTraceFlag() != ECollisionTraceFlag::CTF_UseComplexAsSimple)
				{
					FTransform GeomTransform(GetLocalToWorld());
					BodySetup->AggGeom.GetAggGeom(GeomTransform, GetSelectionColor(FColor(157, 149, 223, 255), IsSelected(), IsHovered()).ToFColor(true), NULL, false, false, AlwaysHasVelocity(), ViewIndex, Collector);
				}

				// Render bounds
				RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
			}
		}
#endif
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bDynamicRelevance = true;
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint(void) const
	{
		return(sizeof(*this) + GetAllocatedSize());
	}

	uint32 GetAllocatedSize(void) const
	{
		return(FPrimitiveSceneProxy::GetAllocatedSize());
	}


#if RHI_RAYTRACING
	virtual bool IsRayTracingRelevant() const override { return true; }

	virtual bool HasRayTracingRepresentation() const override { return true; }

	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override final
	{
		if (!CVarRayTracingProceduralMesh.GetValueOnRenderThread())
		{
			return;
		}

		TConstArrayView<const FSceneView*> Views = Collector.GetViews();
		const uint32 VisibilityMap = Collector.GetVisibilityMap();

		// RT geometry will be generated based on first active view and then reused for all other views
		// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
		const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
		checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

		const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

		for (int32 SegmentIndex = 0; SegmentIndex < Sections.Num(); ++SegmentIndex)
		{
			const FProcMeshProxySection* Section = Sections[SegmentIndex];
			if (Section != nullptr && Section->bSectionVisible)
			{
				FMaterialRenderProxy* MaterialProxy = Section->Material->GetRenderProxy();
				
				if (Section->RayTracingGeometry.IsValid())
				{
					check(Section->RayTracingGeometry.GetInitializer().IndexBuffer.IsValid());

					FRayTracingInstance RayTracingInstance;
					RayTracingInstance.Geometry = &Section->RayTracingGeometry;
					RayTracingInstance.InstanceTransforms.Add(GetLocalToWorld());

					uint32 SectionIdx = 0;
					FMeshBatch MeshBatch;

					MeshBatch.VertexFactory = &Section->VertexFactory;
					MeshBatch.SegmentIndex = 0;
					MeshBatch.MaterialRenderProxy = Section->Material->GetRenderProxy();
					MeshBatch.ReverseCulling = IsLocalToWorldDeterminantNegative();
					MeshBatch.Type = PT_TriangleList;
					MeshBatch.DepthPriorityGroup = SDPG_World;
					MeshBatch.bCanApplyViewModeOverrides = false;
					MeshBatch.CastRayTracedShadow = IsShadowCast(FirstActiveView);

					FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
					BatchElement.IndexBuffer = &Section->IndexBuffer;

					FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
					FPrimitiveUniformShaderParametersBuilder Builder;
					BuildUniformShaderParameters(Builder);
					DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);

					BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

					BatchElement.FirstIndex = 0;
					BatchElement.NumPrimitives = Section->IndexBuffer.Indices.Num() / 3;
					BatchElement.MinVertexIndex = 0;
					BatchElement.MaxVertexIndex = Section->VertexBuffers.PositionVertexBuffer.GetNumVertices() - 1;

					RayTracingInstance.Materials.Add(MeshBatch);
					
					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
					{
						if ((VisibilityMap & (1 << ViewIndex)) == 0)
						{
							continue;
						}

						Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
					}
				}
			}
		}
	}
	
#endif

private:
	/** Array of sections */
	TArray<FProcMeshProxySection*> Sections;

	UBodySetup* BodySetup;

	FMaterialRelevance MaterialRelevance;
};

//////////////////////////////////////////////////////////////////////////


UProceduralMeshComponent::UProceduralMeshComponent(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	bUseComplexAsSimpleCollision = true;
}

void UProceduralMeshComponent::PostLoad()
{
	Super::PostLoad();

	if (ProcMeshBodySetup && IsTemplate())
	{
		ProcMeshBodySetup->SetFlags(RF_Public | RF_ArchetypeObject);
	}
}

void UProceduralMeshComponent::CreateMeshSection_LinearColor(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<int32>& Triangles, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FVector2D>& UV1, const TArray<FVector2D>& UV2, const TArray<FVector2D>& UV3, const TArray<FLinearColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents, bool bCreateCollision, bool bSRGBConversion)
{
	// Convert FLinearColors to FColors
	TArray<FColor> Colors;
	if (VertexColors.Num() > 0)
	{
		Colors.SetNum(VertexColors.Num());

		for (int32 ColorIdx = 0; ColorIdx < VertexColors.Num(); ColorIdx++)
		{
			Colors[ColorIdx] = VertexColors[ColorIdx].ToFColor(bSRGBConversion);
		}
	}

	CreateMeshSection(SectionIndex, Vertices, Triangles, Normals, UV0, UV1, UV2, UV3, Colors, Tangents, bCreateCollision);
}

void UProceduralMeshComponent::CreateMeshSection(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<int32>& Triangles, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FVector2D>& UV1, const TArray<FVector2D>& UV2, const TArray<FVector2D>& UV3, const TArray<FColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents, bool bCreateCollision)
{
	SCOPE_CYCLE_COUNTER(STAT_ProcMesh_CreateMeshSection);

	// Ensure sections array is long enough
	if (SectionIndex >= ProcMeshSections.Num())
	{
		ProcMeshSections.SetNum(SectionIndex + 1, EAllowShrinking::No);
	}

	// Reset this section (in case it already existed)
	FProcMeshSection& NewSection = ProcMeshSections[SectionIndex];
	NewSection.Reset();

	// Copy data to vertex buffer
	const int32 NumVerts = Vertices.Num();
	NewSection.ProcVertexBuffer.Reset();
	NewSection.ProcVertexBuffer.AddUninitialized(NumVerts);
	for (int32 VertIdx = 0; VertIdx < NumVerts; VertIdx++)
	{
		FProcMeshVertex& Vertex = NewSection.ProcVertexBuffer[VertIdx];

		Vertex.Position = Vertices[VertIdx];
		Vertex.Normal = (Normals.Num() == NumVerts) ? Normals[VertIdx] : FVector(0.f, 0.f, 1.f);
		Vertex.UV0 = (UV0.Num() == NumVerts) ? UV0[VertIdx] : FVector2D(0.f, 0.f);
		Vertex.UV1 = (UV1.Num() == NumVerts) ? UV1[VertIdx] : FVector2D(0.f, 0.f);
		Vertex.UV2 = (UV2.Num() == NumVerts) ? UV2[VertIdx] : FVector2D(0.f, 0.f);
		Vertex.UV3 = (UV3.Num() == NumVerts) ? UV3[VertIdx] : FVector2D(0.f, 0.f);
		Vertex.Color = (VertexColors.Num() == NumVerts) ? VertexColors[VertIdx] : FColor(255, 255, 255);
		Vertex.Tangent = (Tangents.Num() == NumVerts) ? Tangents[VertIdx] : FProcMeshTangent();

		// Update bounding box
		NewSection.SectionLocalBox += Vertex.Position;
	}

	// Get triangle indices, clamping to vertex range
	const int32 MaxIndex = NumVerts - 1;
	const auto GetTriIndices = [&Triangles, MaxIndex](int32 Idx)
	{
		return TTuple<int32, int32, int32>(FMath::Min(Triangles[Idx    ], MaxIndex),
										   FMath::Min(Triangles[Idx + 1], MaxIndex),
			                               FMath::Min(Triangles[Idx + 2], MaxIndex));
	};

	const int32 NumTriIndices = (Triangles.Num() / 3) * 3; // Ensure number of triangle indices is multiple of three

	// Detect degenerate triangles, i.e. non-unique vertex indices within the same triangle
	int32 NumDegenerateTriangles = 0;
	for (int32 IndexIdx = 0; IndexIdx < NumTriIndices; IndexIdx += 3)
	{
		int32 a, b, c;
		Tie(a, b, c) = GetTriIndices(IndexIdx);
		NumDegenerateTriangles += a == b || a == c || b == c; //-V614
	}
	if (NumDegenerateTriangles > 0)
	{
		UE_LOGF(LogProceduralComponent, Warning, "Detected %d degenerate triangle%ls with non-unique vertex indices for created mesh section in '%ls'; degenerate triangles will be dropped.",
			   NumDegenerateTriangles, NumDegenerateTriangles > 1 ? TEXT("s") : TEXT(""), *GetFullName());
	}

	// Copy index buffer for non-degenerate triangles
	NewSection.ProcIndexBuffer.Reset();
	NewSection.ProcIndexBuffer.AddUninitialized(NumTriIndices - NumDegenerateTriangles * 3);
	int32 CopyIndexIdx = 0;
	for (int32 IndexIdx = 0; IndexIdx < NumTriIndices; IndexIdx += 3)
	{
		int32 a, b, c;
		Tie(a, b, c) = GetTriIndices(IndexIdx);

		if (a != b && a != c && b != c) //-V614
		{
			NewSection.ProcIndexBuffer[CopyIndexIdx++] = a;
			NewSection.ProcIndexBuffer[CopyIndexIdx++] = b;
			NewSection.ProcIndexBuffer[CopyIndexIdx++] = c;
		}
		else
		{
			--NumDegenerateTriangles;
		}
	}
	check(NumDegenerateTriangles == 0);
	check(CopyIndexIdx == NewSection.ProcIndexBuffer.Num());

	NewSection.bEnableCollision = bCreateCollision;

	UpdateLocalBounds(); // Update overall bounds
	UpdateCollision(); // Mark collision as dirty
	MarkRenderStateDirty(); // New section requires recreating scene proxy
}

void UProceduralMeshComponent::UpdateMeshSection_LinearColor(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FVector2D>& UV1, const TArray<FVector2D>& UV2, const TArray<FVector2D>& UV3, const TArray<FLinearColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents, bool bSRGBConversion)
{
	// Convert FLinearColors to FColors
	TArray<FColor> Colors;
	if (VertexColors.Num() > 0)
	{
		Colors.SetNum(VertexColors.Num());

		for (int32 ColorIdx = 0; ColorIdx < VertexColors.Num(); ColorIdx++)
		{
			Colors[ColorIdx] = VertexColors[ColorIdx].ToFColor(bSRGBConversion);
		}
	}

	UpdateMeshSection(SectionIndex, Vertices, Normals, UV0, UV1, UV2, UV3, Colors, Tangents);
}

void UProceduralMeshComponent::UpdateMeshSection(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FVector2D>& UV1, const TArray<FVector2D>& UV2, const TArray<FVector2D>& UV3, const TArray<FColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents)
{
	SCOPE_CYCLE_COUNTER(STAT_ProcMesh_UpdateSectionGT);

	if(SectionIndex < ProcMeshSections.Num())
	{
		FProcMeshSection& Section = ProcMeshSections[SectionIndex];
		const int32 NumVerts = Vertices.Num();
		const int32 PreviousNumVerts = Section.ProcVertexBuffer.Num();

		// See if positions are changing
		const bool bSameVertexCount = PreviousNumVerts == NumVerts;

		// Update bounds, if we are getting new position data
		if (bSameVertexCount)
		{
			Section.SectionLocalBox = Vertices.Num() ? FBox(Vertices) : FBox(EForceInit::ForceInit);

			// Iterate through vertex data, copying in new info
			for (int32 VertIdx = 0; VertIdx < NumVerts; VertIdx++)
			{
				FProcMeshVertex& ModifyVert = Section.ProcVertexBuffer[VertIdx];

				// Position data
				if (Vertices.Num() == NumVerts)
				{
					ModifyVert.Position = Vertices[VertIdx];
				}

				// Normal data
				if (Normals.Num() == NumVerts)
				{
					ModifyVert.Normal = Normals[VertIdx];
				}

				// Tangent data
				if (Tangents.Num() == NumVerts)
				{
					ModifyVert.Tangent = Tangents[VertIdx];
				}

				// UV0 data
				if (UV0.Num() == NumVerts)
				{
					ModifyVert.UV0 = UV0[VertIdx];
				}
				// UV1 data
				if (UV1.Num() == NumVerts)
				{
					ModifyVert.UV1 = UV1[VertIdx];
				}
				// UV2 data
				if (UV2.Num() == NumVerts)
				{
					ModifyVert.UV2 = UV2[VertIdx];
				}
				// UV3 data
				if (UV3.Num() == NumVerts)
				{
					ModifyVert.UV3 = UV3[VertIdx];
				}

				// Color data
				if (VertexColors.Num() == NumVerts)
				{
					ModifyVert.Color = VertexColors[VertIdx];
				}
			}

			// If we have collision enabled on this section, update that too
			if (Section.bEnableCollision)
			{
				TArray<FVector> CollisionPositions;

				// We have one collision mesh for all sections, so need to build array of _all_ positions
				for (const FProcMeshSection& CollisionSection : ProcMeshSections)
				{
					// If section has collision, copy it
					if (CollisionSection.bEnableCollision)
					{
						for (int32 VertIdx = 0; VertIdx < CollisionSection.ProcVertexBuffer.Num(); VertIdx++)
						{
							CollisionPositions.Add(CollisionSection.ProcVertexBuffer[VertIdx].Position);
						}
					}
				}

				// Pass new positions to trimesh
				BodyInstance.UpdateTriMeshVertices(CollisionPositions);
			}

			// If we have a valid proxy and it is not pending recreation
			if (SceneProxy && !IsRenderStateDirty())
			{
				// Create data to update section
				FProcMeshSectionUpdateData* SectionData = new FProcMeshSectionUpdateData;
				SectionData->TargetSection = SectionIndex;
				SectionData->NewVertexBuffer = Section.ProcVertexBuffer;

				// Enqueue command to send to render thread
				FProceduralMeshSceneProxy* ProcMeshSceneProxy = (FProceduralMeshSceneProxy*)SceneProxy;
				ENQUEUE_RENDER_COMMAND(FProcMeshSectionUpdate)
				([ProcMeshSceneProxy, SectionData](FRHICommandListImmediate& RHICmdList) { ProcMeshSceneProxy->UpdateSection_RenderThread(RHICmdList, SectionData); });
			}

			UpdateLocalBounds();		 // Update overall bounds
			MarkRenderTransformDirty();  // Need to send new bounds to render thread
		}
		else
		{
			UE_LOGF(LogProceduralComponent, Error, "Trying to update a procedural mesh component section with a different number of vertices [Previous: %i, New: %i] (clear and recreate mesh section instead)", PreviousNumVerts, NumVerts);
		}
	}
}

void UProceduralMeshComponent::ClearMeshSection(int32 SectionIndex)
{
	if (SectionIndex < ProcMeshSections.Num())
	{
		ProcMeshSections[SectionIndex].Reset();
		UpdateLocalBounds();
		UpdateCollision();
		MarkRenderStateDirty();
	}
}

void UProceduralMeshComponent::ClearAllMeshSections()
{
	ProcMeshSections.Empty();
	UpdateLocalBounds();
	UpdateCollision();
	MarkRenderStateDirty();
}

void UProceduralMeshComponent::SetMeshSectionVisible(int32 SectionIndex, bool bNewVisibility)
{
	if(SectionIndex < ProcMeshSections.Num())
	{
		// Set game thread state
		ProcMeshSections[SectionIndex].bSectionVisible = bNewVisibility;

		if (SceneProxy)
		{
			// Enqueue command to modify render thread info
			FProceduralMeshSceneProxy* ProcMeshSceneProxy = (FProceduralMeshSceneProxy*)SceneProxy;
			ENQUEUE_RENDER_COMMAND(FProcMeshSectionVisibilityUpdate)(
				[ProcMeshSceneProxy, SectionIndex, bNewVisibility](FRHICommandListImmediate& RHICmdList)
				{
					ProcMeshSceneProxy->SetSectionVisibility_RenderThread(SectionIndex, bNewVisibility);
				});
		}
	}
}

bool UProceduralMeshComponent::IsMeshSectionVisible(int32 SectionIndex) const
{
	return (SectionIndex < ProcMeshSections.Num()) ? ProcMeshSections[SectionIndex].bSectionVisible : false;
}

int32 UProceduralMeshComponent::GetNumSections() const
{
	return ProcMeshSections.Num();
}

void UProceduralMeshComponent::AddCollisionConvexMesh(TArray<FVector> ConvexVerts)
{
	if(ConvexVerts.Num() >= 4)
	{ 
		// New element
		FKConvexElem NewConvexElem;
		// Copy in vertex info
		NewConvexElem.VertexData = ConvexVerts;
		// Update bounding box
		NewConvexElem.ElemBox = FBox(ConvexVerts);
		// Add to array of convex elements
		CollisionConvexElems.Add(NewConvexElem);
		// Refresh collision
		UpdateCollision();
	}
}

void UProceduralMeshComponent::ClearCollisionConvexMeshes()
{
	// Empty simple collision info
	CollisionConvexElems.Empty();
	// Refresh collision
	UpdateCollision();
}

void UProceduralMeshComponent::SetCollisionConvexMeshes(const TArray< TArray<FVector> >& ConvexMeshes)
{
	CollisionConvexElems.Reset();

	// Create element for each convex mesh
	for (int32 ConvexIndex = 0; ConvexIndex < ConvexMeshes.Num(); ConvexIndex++)
	{
		FKConvexElem NewConvexElem;
		NewConvexElem.VertexData = ConvexMeshes[ConvexIndex];
		NewConvexElem.ElemBox = FBox(NewConvexElem.VertexData);

		CollisionConvexElems.Add(NewConvexElem);
	}

	UpdateCollision();
}


void UProceduralMeshComponent::UpdateLocalBounds()
{
	FBox LocalBox(ForceInit);

	for (const FProcMeshSection& Section : ProcMeshSections)
	{
		LocalBox += Section.SectionLocalBox;
	}

	LocalBounds = LocalBox.IsValid ? FBoxSphereBounds(LocalBox) : FBoxSphereBounds(FVector::ZeroVector, FVector::ZeroVector, 0); // fallback to reset box sphere bounds

	// Update global bounds
	UpdateBounds();
	// Need to send to render thread
	MarkRenderTransformDirty();
}

FPrimitiveSceneProxy* UProceduralMeshComponent::CreateSceneProxy()
{
	SCOPE_CYCLE_COUNTER(STAT_ProcMesh_CreateSceneProxy);

	return new FProceduralMeshSceneProxy(this);
}

int32 UProceduralMeshComponent::GetNumMaterials() const
{
	return ProcMeshSections.Num();
}


FProcMeshSection* UProceduralMeshComponent::GetProcMeshSection(int32 SectionIndex)
{
	if (SectionIndex < ProcMeshSections.Num())
	{
		return &ProcMeshSections[SectionIndex];
	}
	else
	{
		return nullptr;
	}
}


void UProceduralMeshComponent::SetProcMeshSection(int32 SectionIndex, const FProcMeshSection& Section)
{
	// Ensure sections array is long enough
	if (SectionIndex >= ProcMeshSections.Num())
	{
		ProcMeshSections.SetNum(SectionIndex + 1, EAllowShrinking::No);
	}

	ProcMeshSections[SectionIndex] = Section;

	UpdateLocalBounds(); // Update overall bounds
	UpdateCollision(); // Mark collision as dirty
	MarkRenderStateDirty(); // New section requires recreating scene proxy
}

FBoxSphereBounds UProceduralMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBoxSphereBounds Ret(LocalBounds.TransformBy(LocalToWorld));

	Ret.BoxExtent *= BoundsScale;
	Ret.SphereRadius *= BoundsScale;

	return Ret;
}

bool UProceduralMeshComponent::GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const
{
	for (const FProcMeshSection& Section : ProcMeshSections)
	{
		if (Section.bEnableCollision)
		{
			OutTriMeshEstimates.VerticeCount += Section.ProcVertexBuffer.Num();
		}
	}

	return true;
}

bool UProceduralMeshComponent::GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData)
{
	int32 VertexBase = 0; // Base vertex index for current section

	// See if we should copy UVs
	bool bCopyUVs = UPhysicsSettings::Get()->bSupportUVFromHitResults; 
	if (bCopyUVs)
	{
		CollisionData->UVs.AddZeroed(1); // only one UV channel
	}

	// For each section..
	for (int32 SectionIdx = 0; SectionIdx < ProcMeshSections.Num(); SectionIdx++)
	{
		FProcMeshSection& Section = ProcMeshSections[SectionIdx];
		// Do we have collision enabled?
		if (Section.bEnableCollision)
		{
			// Copy vert data
			for (int32 VertIdx = 0; VertIdx < Section.ProcVertexBuffer.Num(); VertIdx++)
			{
				CollisionData->Vertices.Add((FVector3f)Section.ProcVertexBuffer[VertIdx].Position);

				// Copy UV if desired
				if (bCopyUVs)
				{
					CollisionData->UVs[0].Add(Section.ProcVertexBuffer[VertIdx].UV0);
				}
			}

			// Copy triangle data
			const int32 NumTriangles = Section.ProcIndexBuffer.Num() / 3;
			for (int32 TriIdx = 0; TriIdx < NumTriangles; TriIdx++)
			{
				// Need to add base offset for indices
				FTriIndices Triangle;
				Triangle.v0 = Section.ProcIndexBuffer[(TriIdx * 3) + 0] + VertexBase;
				Triangle.v1 = Section.ProcIndexBuffer[(TriIdx * 3) + 1] + VertexBase;
				Triangle.v2 = Section.ProcIndexBuffer[(TriIdx * 3) + 2] + VertexBase;
				CollisionData->Indices.Add(Triangle);

				// Also store material info
				CollisionData->MaterialIndices.Add(SectionIdx);
			}

			// Remember the base index that new verts will be added from in next section
			VertexBase = CollisionData->Vertices.Num();
		}
	}

	CollisionData->bFlipNormals = true;
	CollisionData->bDeformableMesh = true;
	CollisionData->bFastCook = true;

	return true;
}

bool UProceduralMeshComponent::ContainsPhysicsTriMeshData(bool InUseAllTriData) const
{
	for (const FProcMeshSection& Section : ProcMeshSections)
	{
		if (Section.ProcIndexBuffer.Num() >= 3 && Section.bEnableCollision)
		{
			return true;
		}
	}

	return false;
}

UBodySetup* UProceduralMeshComponent::CreateBodySetupHelper()
{
	// The body setup in a template needs to be public since the property is Tnstanced and thus is the archetype of the instance meaning there is a direct reference
	UBodySetup* NewBodySetup = NewObject<UBodySetup>(this, NAME_None, (IsTemplate() ? RF_Public | RF_ArchetypeObject : RF_NoFlags));
	NewBodySetup->BodySetupGuid = FGuid::NewGuid();

	NewBodySetup->bGenerateMirroredCollision = false;
	NewBodySetup->bDoubleSidedGeometry = true;
	NewBodySetup->CollisionTraceFlag = bUseComplexAsSimpleCollision ? CTF_UseComplexAsSimple : CTF_UseDefault;

	return NewBodySetup;
}

void UProceduralMeshComponent::CreateProcMeshBodySetup()
{
	if (ProcMeshBodySetup == nullptr)
	{
		ProcMeshBodySetup = CreateBodySetupHelper();
	}
}

void UProceduralMeshComponent::UpdateCollision()
{
	SCOPE_CYCLE_COUNTER(STAT_ProcMesh_UpdateCollision);

	UWorld* World = GetWorld();
	const bool bUseAsyncCook = World && World->IsGameWorld() && bUseAsyncCooking;

	if(bUseAsyncCook)
	{
		// Abort all previous ones still standing
		for (UBodySetup* OldBody : AsyncBodySetupQueue)
		{
			OldBody->AbortPhysicsMeshAsyncCreation();
		}

		AsyncBodySetupQueue.Add(CreateBodySetupHelper());
	}
	else
	{
		AsyncBodySetupQueue.Empty();	//If for some reason we modified the async at runtime, just clear any pending async body setups
		CreateProcMeshBodySetup();
	}
	
	UBodySetup* UseBodySetup = bUseAsyncCook ? AsyncBodySetupQueue.Last() : ProcMeshBodySetup;

	// Fill in simple collision convex elements
	UseBodySetup->AggGeom.ConvexElems = CollisionConvexElems;

	// Set trace flag
	UseBodySetup->CollisionTraceFlag = bUseComplexAsSimpleCollision ? CTF_UseComplexAsSimple : CTF_UseDefault;

	if(bUseAsyncCook)
	{
		UseBodySetup->CreatePhysicsMeshesAsync(FOnAsyncPhysicsCookFinished::CreateUObject(this, &UProceduralMeshComponent::FinishPhysicsAsyncCook, UseBodySetup));
	}
	else
	{
		// New GUID as collision has changed
		UseBodySetup->BodySetupGuid = FGuid::NewGuid();
		// Also we want cooked data for this
		UseBodySetup->bHasCookedCollisionData = true;
		UseBodySetup->InvalidatePhysicsData();
		UseBodySetup->CreatePhysicsMeshes();
		RecreatePhysicsState();
	}
}

void UProceduralMeshComponent::FinishPhysicsAsyncCook(bool bSuccess, UBodySetup* FinishedBodySetup)
{
	TArray<UBodySetup*> NewQueue;
	NewQueue.Reserve(AsyncBodySetupQueue.Num());

	int32 FoundIdx;
	if(AsyncBodySetupQueue.Find(FinishedBodySetup, FoundIdx))
	{
		// Note: currently no-cook-needed is reported identically to cook failed.
		// Checking AggGeom.ConvexElems & ContainsPhysicsTriMeshData here is a hack to distinguish the no-cook-needed case
		// These checks mirror those from UBodySetup::GetCookInfo.
		// TODO: remove this hack to distinguish the no-cook-needed case when/if that is no longer identical to the cook failed case
		const ECollisionTraceFlag BodyCollisionType = FinishedBodySetup->GetCollisionTraceFlag();
		const bool bEmptySimpleCollision = FinishedBodySetup->AggGeom.ConvexElems.Num() == 0;
		const bool bEmptyComplexCollision = !ContainsPhysicsTriMeshData(false);
		const bool bNoCookNeeded =
			(BodyCollisionType == CTF_UseSimpleAsComplex && bEmptySimpleCollision) ||
			(BodyCollisionType == CTF_UseComplexAsSimple && bEmptyComplexCollision) ||
			(BodyCollisionType == CTF_UseSimpleAndComplex && bEmptySimpleCollision && bEmptyComplexCollision);
		if (bSuccess || bNoCookNeeded)
		{
			//The new body was found in the array meaning it's newer so use it
			ProcMeshBodySetup = FinishedBodySetup;
			RecreatePhysicsState();

			//remove any async body setups that were requested before this one
			for (int32 AsyncIdx = FoundIdx + 1; AsyncIdx < AsyncBodySetupQueue.Num(); ++AsyncIdx)
			{
				NewQueue.Add(AsyncBodySetupQueue[AsyncIdx]);
			}

			AsyncBodySetupQueue = NewQueue;
		}
		else
		{
			AsyncBodySetupQueue.RemoveAt(FoundIdx);
		}
	}
}

UBodySetup* UProceduralMeshComponent::GetBodySetup()
{
	CreateProcMeshBodySetup();
	return ProcMeshBodySetup;
}

UMaterialInterface* UProceduralMeshComponent::GetMaterialFromCollisionFaceIndex(int32 FaceIndex, int32& SectionIndex) const
{
	UMaterialInterface* Result = nullptr;
	SectionIndex = 0;

	if (FaceIndex >= 0)
	{
		// Look for element that corresponds to the supplied face
		int32 TotalFaceCount = 0;
		for (int32 SectionIdx = 0; SectionIdx < ProcMeshSections.Num(); SectionIdx++)
		{
			const FProcMeshSection& Section = ProcMeshSections[SectionIdx];
			int32 NumFaces = Section.ProcIndexBuffer.Num() / 3;
			TotalFaceCount += NumFaces;

			if (FaceIndex < TotalFaceCount)
			{
				// Grab the material
				Result = GetMaterial(SectionIdx);
				SectionIndex = SectionIdx;
				break;
			}
		}
	}

	return Result;
}

