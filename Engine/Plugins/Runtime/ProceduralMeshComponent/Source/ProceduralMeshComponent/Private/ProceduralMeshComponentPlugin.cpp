// Copyright Epic Games, Inc. All Rights Reserved.

#include "IProceduralMeshComponentPlugin.h"

class FProceduralMeshComponentPlugin : public IProceduralMeshComponentPlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FProceduralMeshComponentPlugin, ProceduralMeshComponent )



void FProceduralMeshComponentPlugin::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
}


void FProceduralMeshComponentPlugin::ShutdownModule()
{
	
}



