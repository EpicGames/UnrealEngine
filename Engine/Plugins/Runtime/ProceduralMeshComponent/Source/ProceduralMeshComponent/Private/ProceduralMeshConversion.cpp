// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralMeshConversion.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "ProceduralMeshComponent.h"
#include "StaticMeshAttributes.h"


TMap<UMaterialInterface*, FPolygonGroupID> BuildMaterialMap(UProceduralMeshComponent* ProcMeshComp, FMeshDescription& MeshDescription)
{
	TMap<UMaterialInterface*, FPolygonGroupID> UniqueMaterials;
	const int32 NumSections = ProcMeshComp->GetNumSections();
	UniqueMaterials.Reserve(NumSections);

	FStaticMeshAttributes AttributeGetter(MeshDescription);
	TPolygonGroupAttributesRef<FName> PolygonGroupNames = AttributeGetter.GetPolygonGroupMaterialSlotNames();
	for (int32 SectionIdx = 0; SectionIdx < NumSections; SectionIdx++)
	{
		FProcMeshSection *ProcSection =
			ProcMeshComp->GetProcMeshSection(SectionIdx);
		UMaterialInterface *Material = ProcMeshComp->GetMaterial(SectionIdx);
		if (Material == nullptr)
		{
			Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}

		if (!UniqueMaterials.Contains(Material))
		{
			FPolygonGroupID NewPolygonGroup = MeshDescription.CreatePolygonGroup();
			UniqueMaterials.Add(Material, NewPolygonGroup);
			PolygonGroupNames[NewPolygonGroup] = Material->GetFName();
		}
	}
	return UniqueMaterials;
}

namespace
{
	struct FMeshSectionAttributeData
	{
		TArray<FVector> Positions;
		TArray<int32>   Triangles;
		TArray<FVector> Normals;
		TArray<FVector2D> UV0;
		TArray<FColor> VertexColors;
		TArray<FProcMeshTangent> Tangents;
		TMap<FVertexInstanceID, int32> IndexOfInstances;
	};
}

void MeshDescriptionToProcMesh( const FMeshDescription& MeshDescription, UProceduralMeshComponent* ProcMeshComp )
{
	ProcMeshComp->ClearAllMeshSections();

	FStaticMeshConstAttributes AttributeGetter(MeshDescription);
	TPolygonGroupAttributesConstRef<FName> PolygonGroupNames = AttributeGetter.GetPolygonGroupMaterialSlotNames();
	TVertexAttributesConstRef<FVector3f> VertexPositions = AttributeGetter.GetVertexPositions();
	TVertexInstanceAttributesConstRef<FVector3f> Tangents = AttributeGetter.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> BinormalSigns = AttributeGetter.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesConstRef<FVector3f> Normals = AttributeGetter.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector4f> Colors = AttributeGetter.GetVertexInstanceColors();
	TVertexInstanceAttributesConstRef<FVector2f> UVs = AttributeGetter.GetVertexInstanceUVs();

	const int32 NumSections = ProcMeshComp->GetNumSections();

	TMap<FPolygonGroupID, FMeshSectionAttributeData> GroupedData;
	for ( const FTriangleID TriangleID : MeshDescription.Triangles().GetElementIDs() )
	{
		FPolygonGroupID PolygonGroupID = MeshDescription.GetTrianglePolygonGroup( TriangleID );
		FMeshSectionAttributeData& SectionData = GroupedData.FindOrAdd(PolygonGroupID);
		for ( const FVertexInstanceID& VertexInstanceID : MeshDescription.GetTriangleVertexInstances( TriangleID ) )
		{
			auto* FoundVertexIndex = SectionData.IndexOfInstances.Find( VertexInstanceID );
			if ( FoundVertexIndex ){
				SectionData.Triangles.Add( *FoundVertexIndex );
			}
			else
			{
				FVertexID VertexID = MeshDescription.GetVertexInstanceVertex( VertexInstanceID );
				int32 NewVertexIndex = SectionData.IndexOfInstances.Num();
				SectionData.IndexOfInstances.Add( VertexInstanceID,  NewVertexIndex );
				SectionData.Positions.Add( (FVector)VertexPositions[VertexID] );
				SectionData.Triangles.Add( NewVertexIndex );
				SectionData.Normals.Add( (FVector)Normals[VertexInstanceID] );
				SectionData.UV0.Add( FVector2D(UVs[VertexInstanceID]) );
				SectionData.Tangents.Add( FProcMeshTangent( (FVector)Tangents[VertexInstanceID], BinormalSigns[VertexInstanceID] < 0.f ) );
			}
		}
	}
	int32 SectionIndex{0};
	for ( auto& MapEntry : GroupedData )
	{
		auto& SectionData = MapEntry.Value;
		ProcMeshComp->CreateMeshSection(SectionIndex++,
										SectionData.Positions,
										SectionData.Triangles,
										SectionData.Normals,
										SectionData.UV0,
										SectionData.VertexColors,
										SectionData.Tangents,
										true);
	}
}

FMeshDescription BuildMeshDescription( UProceduralMeshComponent* ProcMeshComp )
{
	FMeshDescription MeshDescription;

	FStaticMeshAttributes AttributeGetter(MeshDescription);
	AttributeGetter.Register();

	TPolygonGroupAttributesRef<FName> PolygonGroupNames = AttributeGetter.GetPolygonGroupMaterialSlotNames();
	TVertexAttributesRef<FVector3f> VertexPositions = AttributeGetter.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> Tangents = AttributeGetter.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> BinormalSigns = AttributeGetter.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector3f> Normals = AttributeGetter.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector4f> Colors = AttributeGetter.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> UVs = AttributeGetter.GetVertexInstanceUVs();

	// Materials to apply to new mesh
	const int32 NumSections = ProcMeshComp->GetNumSections();
	int32 VertexCount = 0;
	int32 VertexInstanceCount = 0;
	int32 PolygonCount = 0;

	TMap<UMaterialInterface*, FPolygonGroupID> UniqueMaterials = BuildMaterialMap(ProcMeshComp, MeshDescription);
	TArray<FPolygonGroupID> PolygonGroupForSection;
	PolygonGroupForSection.Reserve(NumSections);

	// Calculate the totals for each ProcMesh element type
	for (int32 SectionIdx = 0; SectionIdx < NumSections; SectionIdx++)
	{
		FProcMeshSection *ProcSection =
			ProcMeshComp->GetProcMeshSection(SectionIdx);
		VertexCount += ProcSection->ProcVertexBuffer.Num();
		VertexInstanceCount += ProcSection->ProcIndexBuffer.Num();
		PolygonCount += ProcSection->ProcIndexBuffer.Num() / 3;
	}
	MeshDescription.ReserveNewVertices(VertexCount);
	MeshDescription.ReserveNewVertexInstances(VertexInstanceCount);
	MeshDescription.ReserveNewPolygons(PolygonCount);
	MeshDescription.ReserveNewEdges(PolygonCount * 2);
	UVs.SetNumChannels(4);

	// Create the Polygon Groups
	for (int32 SectionIdx = 0; SectionIdx < NumSections; SectionIdx++)
	{
		FProcMeshSection *ProcSection =
			ProcMeshComp->GetProcMeshSection(SectionIdx);
		UMaterialInterface *Material = ProcMeshComp->GetMaterial(SectionIdx);
		if (Material == nullptr)
		{
			Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}

		FPolygonGroupID *PolygonGroupID = UniqueMaterials.Find(Material);
		check(PolygonGroupID != nullptr);
		PolygonGroupForSection.Add(*PolygonGroupID);
	}


	// Add Vertex and VertexInstance and polygon for each section
	for (int32 SectionIdx = 0; SectionIdx < NumSections; SectionIdx++)
	{
		FProcMeshSection *ProcSection =
			ProcMeshComp->GetProcMeshSection(SectionIdx);
		FPolygonGroupID PolygonGroupID = PolygonGroupForSection[SectionIdx];
		// Create the vertex
		int32 NumVertex = ProcSection->ProcVertexBuffer.Num();
		TMap<int32, FVertexID> VertexIndexToVertexID;
		VertexIndexToVertexID.Reserve(NumVertex);
		for (int32 VertexIndex = 0; VertexIndex < NumVertex; ++VertexIndex)
		{
			FProcMeshVertex &Vert = ProcSection->ProcVertexBuffer[VertexIndex];
			const FVertexID VertexID = MeshDescription.CreateVertex();
			VertexPositions[VertexID] = (FVector3f)Vert.Position;
			VertexIndexToVertexID.Add(VertexIndex, VertexID);
		}
		// Create the VertexInstance
		int32 NumIndices = ProcSection->ProcIndexBuffer.Num();
		int32 NumTri = NumIndices / 3;
		TMap<int32, FVertexInstanceID> IndiceIndexToVertexInstanceID;
		IndiceIndexToVertexInstanceID.Reserve(NumVertex);
		for (int32 IndiceIndex = 0; IndiceIndex < NumIndices; IndiceIndex++)
		{
			const int32 VertexIndex = ProcSection->ProcIndexBuffer[IndiceIndex];
			const FVertexID VertexID = VertexIndexToVertexID[VertexIndex];
			const FVertexInstanceID VertexInstanceID =
				MeshDescription.CreateVertexInstance(VertexID);
			IndiceIndexToVertexInstanceID.Add(IndiceIndex, VertexInstanceID);

			FProcMeshVertex &ProcVertex = ProcSection->ProcVertexBuffer[VertexIndex];

			Tangents[VertexInstanceID] = (FVector3f)ProcVertex.Tangent.TangentX;
			Normals[VertexInstanceID] = (FVector3f)ProcVertex.Normal;
			BinormalSigns[VertexInstanceID] =
				ProcVertex.Tangent.bFlipTangentY ? -1.f : 1.f;

			Colors[VertexInstanceID] = FLinearColor(ProcVertex.Color);

			UVs.Set(VertexInstanceID, 0, FVector2f(ProcVertex.UV0));	// LWC_TODO: Precision loss
			UVs.Set(VertexInstanceID, 1, FVector2f(ProcVertex.UV1));	// LWC_TODO: Precision loss
			UVs.Set(VertexInstanceID, 2, FVector2f(ProcVertex.UV2));	// LWC_TODO: Precision loss
			UVs.Set(VertexInstanceID, 3, FVector2f(ProcVertex.UV3));	// LWC_TODO: Precision loss
		}

		// Create the polygons for this section
		for (int32 TriIdx = 0; TriIdx < NumTri; TriIdx++)
		{
			FVertexID VertexIndexes[3];
			TArray<FVertexInstanceID> VertexInstanceIDs;
			VertexInstanceIDs.SetNum(3);

			for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
			{
				const int32 IndiceIndex = (TriIdx * 3) + CornerIndex;
				const int32 VertexIndex = ProcSection->ProcIndexBuffer[IndiceIndex];
				VertexIndexes[CornerIndex] = VertexIndexToVertexID[VertexIndex];
				VertexInstanceIDs[CornerIndex] =
					IndiceIndexToVertexInstanceID[IndiceIndex];
			}

			// Insert a polygon into the mesh
			MeshDescription.CreatePolygon(PolygonGroupID, VertexInstanceIDs);
		}
	}
	return MeshDescription;
}
