// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ProceduralMeshComponent : ModuleRules
	{
		public ProceduralMeshComponent(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"MeshDescription",
					"RenderCore",
					"RHI",
					"StaticMeshDescription",
					"PhysicsCore"
				}
				);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"Shaders",
				});
		}
	}
}
