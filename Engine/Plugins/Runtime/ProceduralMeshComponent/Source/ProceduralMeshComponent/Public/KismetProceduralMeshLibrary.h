// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "KismetProceduralMeshLibrary.generated.h"

#define UE_API PROCEDURALMESHCOMPONENT_API

class UMaterialInterface;
class UProceduralMeshComponent;
struct FProcMeshTangent;

class UStaticMesh;
class UStaticMeshComponent;

/** Options for creating cap geometry when slicing */
UENUM()
enum class EProcMeshSliceCapOption : uint8
{
	/** Do not create cap geometry */
	NoCap,
	/** Add a new section to ProceduralMesh for cap */
	CreateNewSectionForCap,
	/** Add cap geometry to existing last section */
	UseLastSectionForCap
};

UCLASS(MinimalAPI, meta=(ScriptName="ProceduralMeshLibrary"))
class UKismetProceduralMeshLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/** Generate vertex and index buffer for a simple box, given the supplied dimensions. Normals, UVs and tangents are also generated for each vertex. */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	static UE_API void GenerateBoxMesh(FVector BoxRadius, TArray<FVector>& Vertices, TArray<int32>& Triangles, TArray<FVector>& Normals, TArray<FVector2D>& UVs, TArray<FProcMeshTangent>& Tangents);

	/** 
	 *	Automatically generate normals and tangent vectors for a mesh
	 *	UVs are required for correct tangent generation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh", meta=(AutoCreateRefTerm = "SmoothingGroups,UVs" ))
	static UE_API void CalculateTangentsForMesh(const TArray<FVector>& Vertices, const TArray<int32>& Triangles, const TArray<FVector2D>& UVs, TArray<FVector>& Normals, TArray<FProcMeshTangent>& Tangents);

	/** Add a quad, specified by four indices, to a triangle index buffer as two triangles. */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	static UE_API void ConvertQuadToTriangles(UPARAM(ref) TArray<int32>& Triangles, int32 Vert0, int32 Vert1, int32 Vert2, int32 Vert3);

	/** 
	 *	Generate an index buffer for a grid of quads. 
	 *	@param	NumX			Number of vertices in X direction (must be >= 2)
	 *	@param	NumY			Number of vertices in y direction (must be >= 2)
	 *	@param	bWinding		Reverses winding of indices generated for each quad
	 *	@out	Triangles		Output index buffer
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	static UE_API void CreateGridMeshTriangles(int32 NumX, int32 NumY, bool bWinding, TArray<int32>& Triangles);

	/**
	*	Generate a vertex buffer, index buffer and UVs for a tessellated grid mesh.
	*	@param	NumX			Number of vertices in X direction (must be >= 2)
	*	@param	NumY			Number of vertices in y direction (must be >= 2)
	*	@out	Triangles		Output index buffer
	*	@out	Vertices		Output vertex buffer
	*	@out	UVs				Out UVs
	*	@param	GridSpacing		Size of each quad in world units
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	static UE_API void CreateGridMeshWelded(int32 NumX, int32 NumY, TArray<int32>& Triangles, TArray<FVector>& Vertices, TArray<FVector2D>& UVs, float GridSpacing = 16.0f);

	/**
	*	Generate a vertex buffer, index buffer and UVs for a grid mesh where each quad is split, with standard 0-1 UVs on UV0 and point sampled texel center UVs for UV1.
	*	@param	NumX			Number of vertices in X direction (must be >= 2)
	*	@param	NumY			Number of vertices in y direction (must be >= 2)
	*	@out	Triangles		Output index buffer
	*	@out	Vertices		Output vertex buffer
	*	@out	UVs				Out UVs
	*	@out	UV1s			Out UV1s
	*	@param	GridSpacing		Size of each quad in world units
	*/
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	static UE_API void CreateGridMeshSplit(int32 NumX, int32 NumY, TArray<int32>& Triangles, TArray<FVector>& Vertices, TArray<FVector2D>& UVs, TArray<FVector2D>& UV1s, float GridSpacing = 16.0f);

	/** Grab geometry data from a StaticMesh asset. */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	static UE_API void GetSectionFromStaticMesh(UStaticMesh* InMesh, int32 LODIndex, int32 SectionIndex, TArray<FVector>& Vertices, TArray<int32>& Triangles, TArray<FVector>& Normals, TArray<FVector2D>& UVs, TArray<FProcMeshTangent>& Tangents);

	/** Copy materials from StaticMeshComponent to ProceduralMeshComponent. */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	static UE_API void CopyProceduralMeshFromStaticMeshComponent(UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, UProceduralMeshComponent* ProcMeshComponent, bool bCreateCollision);

	/** Grab geometry data from a ProceduralMeshComponent. */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	static UE_API void GetSectionFromProceduralMesh(UProceduralMeshComponent* InProcMesh, int32 SectionIndex, TArray<FVector>& Vertices, TArray<int32>& Triangles, TArray<FVector>& Normals, TArray<FVector2D>& UVs, TArray<FProcMeshTangent>& Tangents);


	/** 
	 *	Slice the ProceduralMeshComponent (including simple convex collision) using a plane. Optionally create 'cap' geometry. 
	 *	@param	InProcMesh				ProceduralMeshComponent to slice
	 *	@param	PlanePosition			Point on the plane to use for slicing, in world space
	 *	@param	PlaneNormal				Normal of plane used for slicing. Geometry on the positive side of the plane will be kept.
	 *	@param	bCreateOtherHalf		If true, an additional ProceduralMeshComponent (OutOtherHalfProcMesh) will be created using the other half of the sliced geometry
	 *	@param	OutOtherHalfProcMesh	If bCreateOtherHalf is set, this is the new component created. Its owner will be the same as the supplied InProcMesh.
	 *	@param	CapOption				If and how to create 'cap' geometry on the slicing plane
	 *	@param	CapMaterial				If creating a new section for the cap, assign this material to that section
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	static UE_API void SliceProceduralMesh(UProceduralMeshComponent* InProcMesh, FVector PlanePosition, FVector PlaneNormal, bool bCreateOtherHalf, UProceduralMeshComponent*& OutOtherHalfProcMesh, EProcMeshSliceCapOption CapOption, UMaterialInterface* CapMaterial);
};

#undef UE_API
