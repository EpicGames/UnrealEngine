// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Components/MeshComponent.h"
#include "ProceduralMeshComponent.generated.h"

#define UE_API PROCEDURALMESHCOMPONENT_API

struct FKConvexElem;

class FPrimitiveSceneProxy;

/**
*	Struct used to specify a tangent vector for a vertex
*	The Y tangent is computed from the cross product of the vertex normal (Tangent Z) and the TangentX member.
*/
USTRUCT(BlueprintType)
struct FProcMeshTangent
{
	GENERATED_BODY()
public:

	/** Direction of X tangent for this vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Tangent)
	FVector TangentX;

	/** Bool that indicates whether we should flip the Y tangent when we compute it using cross product */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Tangent)
	bool bFlipTangentY;

	FProcMeshTangent()
		: TangentX(1.f, 0.f, 0.f)
		, bFlipTangentY(false)
	{}

	FProcMeshTangent(float X, float Y, float Z)
		: TangentX(X, Y, Z)
		, bFlipTangentY(false)
	{}

	FProcMeshTangent(FVector InTangentX, bool bInFlipTangentY)
		: TangentX(InTangentX)
		, bFlipTangentY(bInFlipTangentY)
	{}
};

/** One vertex for the procedural mesh, used for storing data internally */
USTRUCT(BlueprintType)
struct FProcMeshVertex
{
	GENERATED_BODY()
public:

	/** Vertex position */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vertex)
	FVector Position;

	/** Vertex normal */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vertex)
	FVector Normal;

	/** Vertex tangent */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vertex)
	FProcMeshTangent Tangent;

	/** Vertex color */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vertex)
	FColor Color;

	/** Vertex texture co-ordinate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vertex)
	FVector2D UV0;

	/** Vertex texture co-ordinate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vertex)
	FVector2D UV1;

	/** Vertex texture co-ordinate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vertex)
	FVector2D UV2;

	/** Vertex texture co-ordinate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vertex)
	FVector2D UV3;


	FProcMeshVertex()
		: Position(0.f, 0.f, 0.f)
		, Normal(0.f, 0.f, 1.f)
		, Tangent(FVector(1.f, 0.f, 0.f), false)
		, Color(255, 255, 255)
		, UV0(0.f, 0.f)
		, UV1(0.f, 0.f)
		, UV2(0.f, 0.f)
		, UV3(0.f, 0.f)
	{}
};

/** One section of the procedural mesh. Each material has its own section. */
USTRUCT()
struct FProcMeshSection
{
	GENERATED_BODY()
public:

	/** Vertex buffer for this section */
	UPROPERTY()
	TArray<FProcMeshVertex> ProcVertexBuffer;

	/** Index buffer for this section */
	UPROPERTY()
	TArray<uint32> ProcIndexBuffer;
	/** Local bounding box of section */
	UPROPERTY()
	FBox SectionLocalBox;

	/** Should we build collision data for triangles in this section */
	UPROPERTY()
	bool bEnableCollision;

	/** Should we display this section */
	UPROPERTY()
	bool bSectionVisible;

	FProcMeshSection()
		: SectionLocalBox(ForceInit)
		, bEnableCollision(false)
		, bSectionVisible(true)
	{}

	/** Reset this section, clear all mesh info. */
	void Reset()
	{
		ProcVertexBuffer.Empty();
		ProcIndexBuffer.Empty();
		SectionLocalBox.Init();
		bEnableCollision = false;
		bSectionVisible = true;
	}
};

/**
*	Component that allows you to specify custom triangle mesh geometry
*	Beware! This feature is experimental and may be substantially changed in future releases.
*/
UCLASS(MinimalAPI, hidecategories = (Object, LOD), meta = (BlueprintSpawnableComponent), ClassGroup = Rendering)
class UProceduralMeshComponent : public UMeshComponent, public IInterface_CollisionDataProvider
{
	GENERATED_BODY()
public:

	UE_API UProceduralMeshComponent(const FObjectInitializer& ObjectInitializer);

	/**
	 *	Create/replace a section for this procedural mesh component.
	 *	This function is deprecated for Blueprints because it uses the unsupported 'Color' type. Use new 'Create Mesh Section' function which uses LinearColor instead.
	 *	@param	SectionIndex		Index of the section to create or replace.
	 *	@param	Vertices			Vertex buffer of all vertex positions to use for this mesh section.
	 *	@param	Triangles			Index buffer indicating which vertices make up each triangle. Length must be a multiple of 3.
	 *	@param	Normals				Optional array of normal vectors for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	UV0					Optional array of texture co-ordinates for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	VertexColors		Optional array of colors for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	Tangents			Optional array of tangent vector for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	bCreateCollision	Indicates whether collision should be created for this section. This adds significant cost.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh", meta = (DeprecatedFunction, DeprecationMessage = "This function is deprecated for Blueprints because it uses the unsupported 'Color' type. Use new 'Create Mesh Section' function which uses LinearColor instead.", DisplayName = "Create Mesh Section FColor", AutoCreateRefTerm = "Normals,UV0,VertexColors,Tangents"))
	void CreateMeshSection(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<int32>& Triangles, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents, bool bCreateCollision)
	{
		TArray<FVector2D> EmptyArray;
		CreateMeshSection(SectionIndex, Vertices, Triangles, Normals, UV0, EmptyArray, EmptyArray, EmptyArray, VertexColors, Tangents, bCreateCollision);
	}

	UE_API void CreateMeshSection(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<int32>& Triangles, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FVector2D>& UV1, const TArray<FVector2D>& UV2, const TArray<FVector2D>& UV3, const TArray<FColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents, bool bCreateCollision);

	/**
	 *	Create/replace a section for this procedural mesh component.
	 *	@param	SectionIndex		Index of the section to create or replace.
	 *	@param	Vertices			Vertex buffer of all vertex positions to use for this mesh section.
	 *	@param	Triangles			Index buffer indicating which vertices make up each triangle. Length must be a multiple of 3.
	 *	@param	Normals				Optional array of normal vectors for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	UV0					Optional array of texture co-ordinates for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	VertexColors		Optional array of colors for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	Tangents			Optional array of tangent vector for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	bCreateCollision	Indicates whether collision should be created for this section. This adds significant cost.
	 *	@param	bSRGBConversion		Whether to do sRGB conversion when converting FLinearColor to FColor
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh", meta = (DisplayName = "Create Mesh Section", AutoCreateRefTerm = "Normals,UV0,UV1,UV2,UV3,VertexColors,Tangents", AdvancedDisplay = "UV1,UV2,UV3"))
	UE_API void CreateMeshSection_LinearColor(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<int32>& Triangles, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FVector2D>& UV1, const TArray<FVector2D>& UV2, const TArray<FVector2D>& UV3,
		const TArray<FLinearColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents, bool bCreateCollision, UPARAM(DisplayName = "SRGB Conversion") bool bSRGBConversion = false);

	void CreateMeshSection_LinearColor(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<int32>& Triangles, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FLinearColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents, bool bCreateCollision, bool bSRGBConversion = false)
	{
		TArray<FVector2D> EmptyArray;
		CreateMeshSection_LinearColor(SectionIndex, Vertices, Triangles, Normals, UV0, EmptyArray, EmptyArray, EmptyArray, VertexColors, Tangents, bCreateCollision, bSRGBConversion);
	}

	/**
	 *	Updates a section of this procedural mesh component. This is faster than CreateMeshSection, but does not let you change topology. Collision info is also updated.
	 *	This function is deprecated for Blueprints because it uses the unsupported 'Color' type. Use new 'Create Mesh Section' function which uses LinearColor instead.
	 *	@param	Vertices			Vertex buffer of all vertex positions to use for this mesh section.
	 *	@param	Normals				Optional array of normal vectors for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	UV0					Optional array of texture co-ordinates for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	VertexColors		Optional array of colors for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	Tangents			Optional array of tangent vector for each vertex. If supplied, must be same length as Vertices array.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh", meta = (DeprecatedFunction, DeprecationMessage = "This function is deprecated for Blueprints because it uses the unsupported 'Color' type. Use new 'Update Mesh Section' function which uses LinearColor instead.", DisplayName = "Update Mesh Section FColor", AutoCreateRefTerm = "Normals,UV0,VertexColors,Tangents"))
	void UpdateMeshSection(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents)
	{
		TArray<FVector2D> EmptyArray;
		UpdateMeshSection(SectionIndex, Vertices, Normals, UV0, EmptyArray, EmptyArray, EmptyArray, VertexColors, Tangents);
	}

	UE_API void UpdateMeshSection(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FVector2D>& UV1, const TArray<FVector2D>& UV2, const TArray<FVector2D>& UV3, const TArray<FColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents);

	/**
	 *	Updates a section of this procedural mesh component. This is faster than CreateMeshSection, but does not let you change topology. Collision info is also updated.
	 *	@param	Vertices			Vertex buffer of all vertex positions to use for this mesh section.
	 *	@param	Normals				Optional array of normal vectors for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	UV0					Optional array of texture co-ordinates for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	VertexColors		Optional array of colors for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	Tangents			Optional array of tangent vector for each vertex. If supplied, must be same length as Vertices array.
	 *	@param	bSRGBConversion		Whether to do sRGB conversion when converting FLinearColor to FColor
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh", meta = (DisplayName = "Update Mesh Section", AutoCreateRefTerm = "Normals,UV0,UV1,UV2,UV3,VertexColors,Tangents", AdvancedDisplay = "UV1,UV2,UV3"))
	UE_API void UpdateMeshSection_LinearColor(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FVector2D>& UV1, const TArray<FVector2D>& UV2, const TArray<FVector2D>& UV3,
		const TArray<FLinearColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents, UPARAM(DisplayName = "SRGB Conversion") bool bSRGBConversion = true);

	void UpdateMeshSection_LinearColor(int32 SectionIndex, const TArray<FVector>& Vertices, const TArray<FVector>& Normals, const TArray<FVector2D>& UV0, const TArray<FLinearColor>& VertexColors, const TArray<FProcMeshTangent>& Tangents, bool bSRGBConversion = true)
	{
		TArray<FVector2D> EmptyArray;
		UpdateMeshSection_LinearColor(SectionIndex, Vertices, Normals, UV0, EmptyArray, EmptyArray, EmptyArray, VertexColors, Tangents, bSRGBConversion);
	}

	/** Clear a section of the procedural mesh. Other sections do not change index. */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	UE_API void ClearMeshSection(int32 SectionIndex);

	/** Clear all mesh sections and reset to empty state */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	UE_API void ClearAllMeshSections();

	/** Control visibility of a particular section */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	UE_API void SetMeshSectionVisible(int32 SectionIndex, bool bNewVisibility);

	/** Returns whether a particular section is currently visible */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	UE_API bool IsMeshSectionVisible(int32 SectionIndex) const;

	/** Returns number of sections currently created for this component */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	UE_API int32 GetNumSections() const;

	/** Add simple collision convex to this component */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	UE_API void AddCollisionConvexMesh(TArray<FVector> ConvexVerts);

	/** Remove collision meshes from this component */
	UFUNCTION(BlueprintCallable, Category = "Components|ProceduralMesh")
	UE_API void ClearCollisionConvexMeshes();

	/** Function to replace _all_ simple collision in one go */
	UE_API void SetCollisionConvexMeshes(const TArray< TArray<FVector> >& ConvexMeshes);

	//~ Begin Interface_CollisionDataProvider Interface
	UE_API virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const override;
	UE_API virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	UE_API virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override;
	virtual bool WantsNegXTriMesh() override{ return false; }
	//~ End Interface_CollisionDataProvider Interface

	/** 
	 *	Controls whether the complex (Per poly) geometry should be treated as 'simple' collision. 
	 *	Should be set to false if this component is going to be given simple collision and simulated.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Procedural Mesh")
	bool bUseComplexAsSimpleCollision;

	/**
	*	Controls whether the physics cooking should be done off the game thread. This should be used when collision geometry doesn't have to be immediately up to date (For example streaming in far away objects)
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Procedural Mesh")
	bool bUseAsyncCooking;

	/** Collision data */
	UPROPERTY(Instanced)
	TObjectPtr<class UBodySetup> ProcMeshBodySetup;

	/** 
	 *	Get pointer to internal data for one section of this procedural mesh component. 
	 *	Note that pointer will becomes invalid if sections are added or removed.
	 */
	UE_API FProcMeshSection* GetProcMeshSection(int32 SectionIndex);

	/** Replace a section with new section geometry */
	UE_API void SetProcMeshSection(int32 SectionIndex, const FProcMeshSection& Section);

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual class UBodySetup* GetBodySetup() override;
	UE_API virtual UMaterialInterface* GetMaterialFromCollisionFaceIndex(int32 FaceIndex, int32& SectionIndex) const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface.
	UE_API virtual int32 GetNumMaterials() const override;
	//~ End UMeshComponent Interface.

	//~ Begin UObject Interface
	UE_API virtual void PostLoad() override;
	//~ End UObject Interface.

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface.

private:

	/** Update LocalBounds member from the local box of each section */
	UE_API void UpdateLocalBounds();
	/** Ensure ProcMeshBodySetup is allocated and configured */
	UE_API void CreateProcMeshBodySetup();
	/** Mark collision data as dirty, and re-create on instance if necessary */
	UE_API void UpdateCollision();
	/** Once async physics cook is done, create needed state */
	UE_API void FinishPhysicsAsyncCook(bool bSuccess, UBodySetup* FinishedBodySetup);

	/** Helper to create new body setup objects */
	UE_API UBodySetup* CreateBodySetupHelper();

	/** Array of sections of mesh */
	UPROPERTY()
	TArray<FProcMeshSection> ProcMeshSections;

	/** Convex shapes used for simple collision */
	UPROPERTY()
	TArray<FKConvexElem> CollisionConvexElems;

	/** Local space bounds of mesh */
	UPROPERTY()
	FBoxSphereBounds LocalBounds;
	
	/** Queue for async body setups that are being cooked */
	UPROPERTY(transient)
	TArray<TObjectPtr<UBodySetup>> AsyncBodySetupQueue;

	friend class FProceduralMeshSceneProxy;
};

#undef UE_API
