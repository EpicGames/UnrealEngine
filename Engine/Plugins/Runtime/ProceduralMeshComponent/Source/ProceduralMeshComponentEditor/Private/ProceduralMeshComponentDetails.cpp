// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralMeshComponentDetails.h"
#include "BodySetupEnums.h"
#include "ProceduralMeshConversion.h"
#include "Engine/StaticMeshSourceData.h"
#include "Misc/PackageName.h"
#include "MeshDescription.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Engine/StaticMesh.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorDirectories.h"
#include "ProceduralMeshComponent.h"
#include "PhysicsEngine/BodySetup.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "AssetRegistry/AssetRegistryModule.h"

#define LOCTEXT_NAMESPACE "ProceduralMeshComponentDetails"

TSharedRef<IDetailCustomization> FProceduralMeshComponentDetails::MakeInstance()
{
	return MakeShareable(new FProceduralMeshComponentDetails);
}

void FProceduralMeshComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	IDetailCategoryBuilder& ProcMeshCategory = DetailBuilder.EditCategory("ProceduralMesh");

	const FText ConvertToStaticMeshText = LOCTEXT("ConvertToStaticMesh", "Create StaticMesh");

	// Cache set of selected things
	SelectedObjectsList = DetailBuilder.GetSelectedObjects();

	ProcMeshCategory.AddCustomRow(ConvertToStaticMeshText, false)
	.NameContent()
	[
		SNullWidget::NullWidget
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	.MaxDesiredWidth(250)
	[
		SNew(SButton)
		.VAlign(VAlign_Center)
		.ToolTipText(LOCTEXT("ConvertToStaticMeshTooltip", "Create a new StaticMesh asset using current geometry from this ProceduralMeshComponent. Does not modify instance."))
		.OnClicked(this, &FProceduralMeshComponentDetails::ClickedOnConvertToStaticMesh)
		.IsEnabled(this, &FProceduralMeshComponentDetails::ConvertToStaticMeshEnabled)
		.Content()
		[
			SNew(STextBlock)
			.Text(ConvertToStaticMeshText)
		]
	];
}

UProceduralMeshComponent* FProceduralMeshComponentDetails::GetFirstSelectedProcMeshComp() const
{
	// Find first selected valid ProcMeshComp
	UProceduralMeshComponent* ProcMeshComp = nullptr;
	for (const TWeakObjectPtr<UObject>& Object : SelectedObjectsList)
	{
		UProceduralMeshComponent* TestProcComp = Cast<UProceduralMeshComponent>(Object.Get());
		// See if this one is good
		if (TestProcComp != nullptr && !TestProcComp->IsTemplate())
		{
			ProcMeshComp = TestProcComp;
			break;
		}
	}

	return ProcMeshComp;
}

bool FProceduralMeshComponentDetails::ConvertToStaticMeshEnabled() const
{
	return GetFirstSelectedProcMeshComp() != nullptr;
}

FReply FProceduralMeshComponentDetails::ClickedOnConvertToStaticMesh()
{
	// Find first selected ProcMeshComp
	UProceduralMeshComponent* ProcMeshComp = GetFirstSelectedProcMeshComp();
	if (ProcMeshComp != nullptr)
	{
		FString NewNameSuggestion = FString(TEXT("ProcMesh"));
		FString DefaultPath;
		const FString DefaultDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
		FPackageName::TryConvertFilenameToLongPackageName(DefaultDirectory, DefaultPath);

		if (DefaultPath.IsEmpty())
		{
			DefaultPath = TEXT("/Game/Meshes");
		}

		FString PackageName = DefaultPath / NewNameSuggestion;
		FString Name;
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(PackageName, TEXT(""), PackageName, Name);

		TSharedPtr<SDlgPickAssetPath> PickAssetPathWidget =
			SNew(SDlgPickAssetPath)
			.Title(LOCTEXT("ConvertToStaticMeshPickName", "Choose New StaticMesh Location"))
			.DefaultAssetPath(FText::FromString(PackageName));

		if (PickAssetPathWidget->ShowModal() == EAppReturnType::Ok)
		{
			// Get the full name of where we want to create the physics asset.
			FString UserPackageName = PickAssetPathWidget->GetFullAssetPath().ToString();
			FName MeshName(*FPackageName::GetLongPackageAssetName(UserPackageName));

			// Check if the user inputed a valid asset name, if they did not, give it the generated default name
			if (MeshName == NAME_None)
			{
				// Use the defaults that were already generated.
				UserPackageName = PackageName;
				MeshName = *Name;
			}


			FMeshDescription MeshDescription = BuildMeshDescription(ProcMeshComp);

			// If we got some valid data.
			if (MeshDescription.Polygons().Num() > 0)
			{
				// Then find/create it.
				UPackage* Package = CreatePackage(*UserPackageName);
				check(Package);

				// Create StaticMesh object
				UStaticMesh* StaticMesh = NewObject<UStaticMesh>(Package, MeshName, RF_Public | RF_Standalone);
				StaticMesh->InitResources();

				StaticMesh->SetLightingGuid();

				// Add source to new StaticMesh
				FStaticMeshSourceModel& SrcModel = StaticMesh->AddSourceModel();
				SrcModel.BuildSettings.bRecomputeNormals = false;
				SrcModel.BuildSettings.bRecomputeTangents = false;
				SrcModel.BuildSettings.bRemoveDegenerates = false;
				SrcModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
				SrcModel.BuildSettings.bUseFullPrecisionUVs = false;
				SrcModel.BuildSettings.bGenerateLightmapUVs = true;
				SrcModel.BuildSettings.SrcLightmapIndex = 0;
				SrcModel.BuildSettings.DstLightmapIndex = 1;
				StaticMesh->CreateMeshDescription(0, MoveTemp(MeshDescription));
				StaticMesh->CommitMeshDescription(0);

				//// SIMPLE COLLISION
				if (!ProcMeshComp->bUseComplexAsSimpleCollision )
				{
					StaticMesh->CreateBodySetup();
					UBodySetup* NewBodySetup = StaticMesh->GetBodySetup();
					NewBodySetup->BodySetupGuid = FGuid::NewGuid();
					NewBodySetup->AggGeom.ConvexElems = ProcMeshComp->ProcMeshBodySetup->AggGeom.ConvexElems;
					NewBodySetup->bGenerateMirroredCollision = false;
					NewBodySetup->bDoubleSidedGeometry = true;
					NewBodySetup->CollisionTraceFlag = CTF_UseDefault;
					NewBodySetup->CreatePhysicsMeshes();
				}

				//// MATERIALS
				TSet<UMaterialInterface*> UniqueMaterials;
				const int32 NumSections = ProcMeshComp->GetNumSections();
				for (int32 SectionIdx = 0; SectionIdx < NumSections; SectionIdx++)
				{
					FProcMeshSection *ProcSection =
						ProcMeshComp->GetProcMeshSection(SectionIdx);
					UMaterialInterface *Material = ProcMeshComp->GetMaterial(SectionIdx);
					UniqueMaterials.Add(Material);
				}
				// Copy materials to new mesh
				for (auto* Material : UniqueMaterials)
				{
					StaticMesh->GetStaticMaterials().Add(FStaticMaterial(Material));
				}

				//Set the Imported version before calling the build
				StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);

				// Build mesh from source
				StaticMesh->Build(false);
				StaticMesh->PostEditChange();

				// Notify asset registry of new asset
				FAssetRegistryModule::AssetCreated(StaticMesh);
			}
		}
	}

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
