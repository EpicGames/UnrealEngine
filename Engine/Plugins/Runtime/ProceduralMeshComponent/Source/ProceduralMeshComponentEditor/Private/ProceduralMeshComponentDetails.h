// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FReply;

class IDetailLayoutBuilder;
class UProceduralMeshComponent;

class FProceduralMeshComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

	/** Handle clicking the convert button */
	FReply ClickedOnConvertToStaticMesh();

	/** Is the convert button enabled */
	bool ConvertToStaticMeshEnabled() const;

	/** Util to get the ProcMeshComp we want to convert */
	UProceduralMeshComponent* GetFirstSelectedProcMeshComp() const;

	/** Cached array of selected objects */
	TArray< TWeakObjectPtr<UObject> > SelectedObjectsList;
};
