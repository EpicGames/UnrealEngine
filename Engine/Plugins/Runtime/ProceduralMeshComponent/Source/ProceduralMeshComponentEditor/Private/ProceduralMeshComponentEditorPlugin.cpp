// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorModule.h"
#include "ProceduralMeshComponent.h"
#include "IProceduralMeshComponentEditorPlugin.h"
#include "ProceduralMeshComponentDetails.h"




class FProceduralMeshComponentEditorPlugin : public IProceduralMeshComponentEditorPlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FProceduralMeshComponentEditorPlugin, ProceduralMeshComponentEditor )



void FProceduralMeshComponentEditorPlugin::StartupModule()
{
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(UProceduralMeshComponent::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FProceduralMeshComponentDetails::MakeInstance));
	}
}


void FProceduralMeshComponentEditorPlugin::ShutdownModule()
{
	
}



