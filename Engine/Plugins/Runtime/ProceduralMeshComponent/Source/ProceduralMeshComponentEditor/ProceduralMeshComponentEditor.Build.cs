// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ProceduralMeshComponentEditor : ModuleRules
	{
        public ProceduralMeshComponentEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                    "Slate",
                    "SlateCore",
                    "Engine",
					"EditorFramework",
                    "UnrealEd",
                    "PropertyEditor",
                    "RenderCore",
                    "RHI",
                    "ProceduralMeshComponent",
                    "MeshDescription",
					"StaticMeshDescription",
                    "AssetTools",
                    "AssetRegistry"
                }
				);
		}
	}
}
